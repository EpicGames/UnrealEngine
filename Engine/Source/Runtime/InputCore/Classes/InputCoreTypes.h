// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Misc/Attribute.h"
#include "InputCoreTypes.generated.h"

INPUTCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogInput, Log, All);


/** Defines the controller hands for tracking.  Could be expanded, as needed, to facilitate non-handheld controllers */
UENUM(BlueprintType)
enum class EControllerHand : uint8
{
	Left,
	Right,
	AnyHand,
	Pad,
	ExternalCamera,
	Gun,
	HMD,
	Chest,
	LeftShoulder,
	RightShoulder,
	LeftElbow,
	RightElbow,
	Waist,
	LeftKnee,
	RightKnee,
	LeftFoot,
	RightFoot,
	Special,

	ControllerHand_Count UMETA(Hidden, DisplayName = "<INVALID>"),
};

enum class EPairedAxis : uint8
{
	Unpaired,			// This key is unpaired
	X,					// This key represents the X axis of its PairedAxisKey
	Y,					// This key represents the Y axis of its PairedAxisKey
	Z,					// This key represents the Z axis of its PairedAxisKey - Currently unused
};

USTRUCT(BlueprintType,Blueprintable)
struct FKey
{
	GENERATED_USTRUCT_BODY()

	FKey()
	{
	}

	FKey(const FName InName)
		: KeyName(InName)
	{
	}

	FKey(const TCHAR* InName)
		: KeyName(FName(InName))
	{
	}

	FKey(const ANSICHAR* InName)
		: KeyName(FName(InName))
	{
	}

	INPUTCORE_API bool IsValid() const;
	INPUTCORE_API bool IsModifierKey() const;
	INPUTCORE_API bool IsGamepadKey() const;
	INPUTCORE_API bool IsTouch() const;
	INPUTCORE_API bool IsMouseButton() const;
	INPUTCORE_API bool IsButtonAxis() const;
	INPUTCORE_API bool IsAxis1D() const;
	INPUTCORE_API bool IsAxis2D() const;
	INPUTCORE_API bool IsAxis3D() const;
	INPUTCORE_API bool IsDigital() const;
	INPUTCORE_API bool IsAnalog() const;
	INPUTCORE_API bool IsVirtual() const;
	INPUTCORE_API bool IsBindableInBlueprints() const;
	INPUTCORE_API bool ShouldUpdateAxisWithoutSamples() const;
	INPUTCORE_API bool IsBindableToActions() const;
	INPUTCORE_API bool IsDeprecated() const;
	INPUTCORE_API bool IsGesture() const;
	INPUTCORE_API FText GetDisplayName(bool bLongDisplayName = true) const;
	INPUTCORE_API FString ToString() const;
	INPUTCORE_API FName GetFName() const;
	INPUTCORE_API FName GetMenuCategory() const;
	INPUTCORE_API EPairedAxis GetPairedAxis() const;
	INPUTCORE_API FKey GetPairedAxisKey() const;
	INPUTCORE_API FKey GetVirtualKey() const;

	/**
	 * @param Other		The key to compare against
	 * @return			True if this FKey's name or its virtual key name (if it has one) is the same as the other.
	 */
	INPUTCORE_API bool IsSameResolvedKey(const FKey& Other) const;

	INPUTCORE_API bool SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot);
	INPUTCORE_API bool ExportTextItem(FString& ValueStr, FKey const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	INPUTCORE_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);
	INPUTCORE_API void PostSerialize(const FArchive& Ar);
	INPUTCORE_API void PostScriptConstruct();

	friend bool operator==(const FKey& KeyA, const FKey& KeyB) { return KeyA.KeyName == KeyB.KeyName; }
	friend bool operator!=(const FKey& KeyA, const FKey& KeyB) { return KeyA.KeyName != KeyB.KeyName; }
	friend bool operator<(const FKey& KeyA, const FKey& KeyB) { return KeyA.KeyName.LexicalLess(KeyB.KeyName); }
	friend uint32 GetTypeHash(const FKey& Key) { return GetTypeHash(Key.KeyName); }

	friend struct EKeys;

	INPUTCORE_API static const TCHAR* SyntheticCharPrefix;

private:

	UPROPERTY(EditAnywhere, Category="Input")
	FName KeyName;

	mutable class TSharedPtr<struct FKeyDetails> KeyDetails;

	void ConditionalLookupKeyDetails() const;
	void ResetKey();
};

template<>
struct TStructOpsTypeTraits<FKey> : public TStructOpsTypeTraitsBase2<FKey>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithPostSerialize = true,
		WithPostScriptConstruct = true,
		WithCopy = true,		// Necessary so that TSharedPtr<FKeyDetails> Data is copied around
	};
};

DECLARE_DELEGATE_RetVal_OneParam(FText, FGetKeyDisplayNameSignature, const FKey);

struct FKeyDetails
{
	enum EKeyFlags
	{
		GamepadKey				 = 1 << 0,
		Touch					 = 1 << 1,
		MouseButton				 = 1 << 2,
		ModifierKey				 = 1 << 3,
		NotBlueprintBindableKey	 = 1 << 4,
		Axis1D					 = 1 << 5,
		Axis3D					 = 1 << 6,
		UpdateAxisWithoutSamples = 1 << 7,
		NotActionBindableKey	 = 1 << 8,
		Deprecated				 = 1 << 9,

		// All axis representations
		ButtonAxis				 = 1 << 10,		// Analog 1D axis emulating a digital button press. E.g. Gamepad right stick up
		Axis2D					 = 1 << 11,

		// Gesture input types such as Flick, Pinch, and Rotate
		Gesture					 = 1 << 12,

		// A key is "Virtual" if it is an abstract key whose actual value may change dependent on the platform.
		// For example, the standard "Accept" button on some platforms may be Gamepad_FaceButton_Down, while on
		// other platforms it may be Gamepad_FaceButton_Right.
		//
		// Virtual keys are typically the most useful when it comes to building UI related code because they allow you 
		// to create experiences and screens displayed to the end user which are consistent with the platform's UI.
		Virtual					 = 1 << 13,

		NoFlags                  = 0,
	};

	INPUTCORE_API FKeyDetails(const FKey InKey, const TAttribute<FText>& InLongDisplayName, const uint32 InKeyFlags = 0, const FName InMenuCategory = NAME_None, const TAttribute<FText>& InShortDisplayName = TAttribute<FText>() );
	INPUTCORE_API FKeyDetails(const FKey InKey, const TAttribute<FText>& InLongDisplayName, const TAttribute<FText>& InShortDisplayName, const uint32 InKeyFlags = 0, const FName InMenuCategory = NAME_None);

	inline bool IsModifierKey() const { return bIsModifierKey != 0; }
	inline bool IsGamepadKey() const { return bIsGamepadKey != 0; }
	inline bool IsTouch() const { return bIsTouch != 0; }
	inline bool IsMouseButton() const { return bIsMouseButton != 0; }
	inline bool IsAxis1D() const { return AxisType == EInputAxisType::Axis1D; }
	inline bool IsAxis2D() const { return AxisType == EInputAxisType::Axis2D; }
	inline bool IsAxis3D() const { return AxisType == EInputAxisType::Axis3D; }
	inline bool IsButtonAxis() const { return AxisType == EInputAxisType::Button; }	// Analog 1D axis emulating a digital button press.
	inline bool IsAnalog() const { return IsAxis1D() || IsAxis2D() || IsAxis3D(); }
	inline bool IsDigital() const { return !IsAnalog(); }
	inline bool IsBindableInBlueprints() const { return bIsBindableInBlueprints != 0; }
	inline bool ShouldUpdateAxisWithoutSamples() const { return bShouldUpdateAxisWithoutSamples != 0; }
	inline bool IsBindableToActions() const { return bIsBindableToActions != 0; }
	inline bool IsGesture() const { return bIsGesture != 0; }
	inline bool IsDeprecated() const { return bIsDeprecated != 0; }
	inline bool IsVirtual() const { return bIsVirtual != 0; }
	inline FName GetMenuCategory() const { return MenuCategory; }
	INPUTCORE_API FText GetDisplayName(const bool bLongDisplayName = true) const;
	inline const FKey& GetKey() const { return Key; }

	// Key pairing
	inline EPairedAxis GetPairedAxis() const { return PairedAxis; }
	inline const FKey& GetPairedAxisKey() const { return PairedAxisKey; }
	
	/**
	 * For virtual Keys, this will return the virtual key value. For non-virtual keys, it will return the "normal" key for easier checking
	 */
	inline const FKey& GetVirtualKey() const { return bIsVirtual ? VirtualKeyValue : Key; }

private:
	friend struct EKeys;

	INPUTCORE_API void CommonInit(const uint32 InKeyFlags);

	enum class EInputAxisType : uint8
	{
		None,
		Button,			// Whilst the physical input is an analog axis the FKey uses it to emulate a digital button.
		Axis1D,
		Axis2D,
		Axis3D,
	};

	FKey  Key;

	// Key pairing
	EPairedAxis PairedAxis = EPairedAxis::Unpaired;		// Paired axis identifier. Lets this key know which axis it represents on the PairedAxisKey
	FKey		PairedAxisKey;							// Paired axis reference. This is the FKey representing the final paired vector axis. Note: NOT the other key in the pairing.

	/**
	 * The actual FKey which should be used at runtime for virtual FKeys
	 */
	FKey VirtualKeyValue;

	FName MenuCategory;

	uint8 bIsModifierKey : 1;
	uint8 bIsGamepadKey : 1;
	uint8 bIsTouch : 1;
	uint8 bIsMouseButton : 1;
	uint8 bIsBindableInBlueprints : 1;
	uint8 bShouldUpdateAxisWithoutSamples : 1;
	uint8 bIsBindableToActions : 1;
	uint8 bIsDeprecated : 1;
	uint8 bIsVirtual : 1;
	uint8 bIsGesture : 1;
	EInputAxisType AxisType;

	TAttribute<FText> LongDisplayName;
	TAttribute<FText> ShortDisplayName;
};

UENUM(BlueprintType)
namespace ETouchIndex
{
	// The number of entries in ETouchIndex must match the number of touch keys defined in EKeys and NUM_TOUCH_KEYS above
	enum Type : int
	{
		Touch1,
		Touch2,
		Touch3,
		Touch4,
		Touch5,
		Touch6,
		Touch7,
		Touch8,
		Touch9,
		Touch10,
		/**
		 * This entry is special.  NUM_TOUCH_KEYS - 1, is used for the cursor so that it's represented
		 * as another finger index, but doesn't overlap with touch input indexes.
		 */
		CursorPointerIndex UMETA(Hidden),
		MAX_TOUCHES UMETA(Hidden)
	};
}

/**
 * Identifies a specific touch contact on a specific input device.
 * Combines device identity (FInputDeviceId) with finger index (ETouchIndex) so that multiple touchscreens can each have independent finger-0, etc.
 */
USTRUCT(BlueprintType, MinimalAPI)
struct FTouchId
{
	GENERATED_BODY()

	FTouchId() = default;

	explicit FTouchId(const FInputDeviceId InDeviceId, const ETouchIndex::Type InTouchIndex)
		: DeviceId(InDeviceId)
		, TouchIndex(InTouchIndex)
	{
	}

	FInputDeviceId GetDeviceId() const
	{
		return DeviceId;
	}

	ETouchIndex::Type GetIndex() const
	{
		return TouchIndex;
	}

	bool operator==(const FTouchId& Other) const = default;

	bool operator!=(const FTouchId& Other) const = default;

	bool IsValid() const
	{
		return DeviceId.IsValid();
	}

	friend uint32 GetTypeHash(const FTouchId& Id)
	{
		return HashCombine(GetTypeHash(Id.DeviceId), GetTypeHash(static_cast<int32>(Id.TouchIndex)));
	}

private:

	/**
	 * The device being touched.
	 */
	UPROPERTY(BlueprintReadOnly, Category = "Input|Touch", meta = (AllowPrivateAccess = "true"))
	FInputDeviceId DeviceId = INPUTDEVICEID_NONE;

	/**
	 * The device-specific index that differentiates this touch from other simultaneous touch inputs on the same device.
	 */
	UPROPERTY(BlueprintReadOnly, Category = "Input|Touch", meta = (AllowPrivateAccess = "true"))
	TEnumAsByte<ETouchIndex::Type> TouchIndex = ETouchIndex::Touch1;
};

UENUM()
namespace EConsoleForGamepadLabels
{
	enum Type : int
	{
		None,
		XBoxOne,
		PS4
	};
}

struct EKeys
{
	static INPUTCORE_API const FKey AnyKey;

	static INPUTCORE_API const FKey MouseX;
	static INPUTCORE_API const FKey MouseY;
	static INPUTCORE_API const FKey Mouse2D;
	static INPUTCORE_API const FKey MouseScrollUp;
	static INPUTCORE_API const FKey MouseScrollDown;
	static INPUTCORE_API const FKey MouseWheelAxis;

	static INPUTCORE_API const FKey LeftMouseButton;
	static INPUTCORE_API const FKey RightMouseButton;
	static INPUTCORE_API const FKey MiddleMouseButton;
	static INPUTCORE_API const FKey ThumbMouseButton;
	static INPUTCORE_API const FKey ThumbMouseButton2;

	static INPUTCORE_API const FKey BackSpace;
	static INPUTCORE_API const FKey Tab;
	static INPUTCORE_API const FKey Enter;
	static INPUTCORE_API const FKey Pause;

	static INPUTCORE_API const FKey CapsLock;
	static INPUTCORE_API const FKey Escape;
	static INPUTCORE_API const FKey SpaceBar;
	static INPUTCORE_API const FKey PageUp;
	static INPUTCORE_API const FKey PageDown;
	static INPUTCORE_API const FKey End;
	static INPUTCORE_API const FKey Home;

	static INPUTCORE_API const FKey Left;
	static INPUTCORE_API const FKey Up;
	static INPUTCORE_API const FKey Right;
	static INPUTCORE_API const FKey Down;

	static INPUTCORE_API const FKey Insert;
	static INPUTCORE_API const FKey Delete;

	static INPUTCORE_API const FKey Zero;
	static INPUTCORE_API const FKey One;
	static INPUTCORE_API const FKey Two;
	static INPUTCORE_API const FKey Three;
	static INPUTCORE_API const FKey Four;
	static INPUTCORE_API const FKey Five;
	static INPUTCORE_API const FKey Six;
	static INPUTCORE_API const FKey Seven;
	static INPUTCORE_API const FKey Eight;
	static INPUTCORE_API const FKey Nine;

	static INPUTCORE_API const FKey A;
	static INPUTCORE_API const FKey B;
	static INPUTCORE_API const FKey C;
	static INPUTCORE_API const FKey D;
	static INPUTCORE_API const FKey E;
	static INPUTCORE_API const FKey F;
	static INPUTCORE_API const FKey G;
	static INPUTCORE_API const FKey H;
	static INPUTCORE_API const FKey I;
	static INPUTCORE_API const FKey J;
	static INPUTCORE_API const FKey K;
	static INPUTCORE_API const FKey L;
	static INPUTCORE_API const FKey M;
	static INPUTCORE_API const FKey N;
	static INPUTCORE_API const FKey O;
	static INPUTCORE_API const FKey P;
	static INPUTCORE_API const FKey Q;
	static INPUTCORE_API const FKey R;
	static INPUTCORE_API const FKey S;
	static INPUTCORE_API const FKey T;
	static INPUTCORE_API const FKey U;
	static INPUTCORE_API const FKey V;
	static INPUTCORE_API const FKey W;
	static INPUTCORE_API const FKey X;
	static INPUTCORE_API const FKey Y;
	static INPUTCORE_API const FKey Z;

	static INPUTCORE_API const FKey NumPadZero;
	static INPUTCORE_API const FKey NumPadOne;
	static INPUTCORE_API const FKey NumPadTwo;
	static INPUTCORE_API const FKey NumPadThree;
	static INPUTCORE_API const FKey NumPadFour;
	static INPUTCORE_API const FKey NumPadFive;
	static INPUTCORE_API const FKey NumPadSix;
	static INPUTCORE_API const FKey NumPadSeven;
	static INPUTCORE_API const FKey NumPadEight;
	static INPUTCORE_API const FKey NumPadNine;

	static INPUTCORE_API const FKey Multiply;
	static INPUTCORE_API const FKey Add;
	static INPUTCORE_API const FKey Subtract;
	static INPUTCORE_API const FKey Decimal;
	static INPUTCORE_API const FKey Divide;

	static INPUTCORE_API const FKey F1;
	static INPUTCORE_API const FKey F2;
	static INPUTCORE_API const FKey F3;
	static INPUTCORE_API const FKey F4;
	static INPUTCORE_API const FKey F5;
	static INPUTCORE_API const FKey F6;
	static INPUTCORE_API const FKey F7;
	static INPUTCORE_API const FKey F8;
	static INPUTCORE_API const FKey F9;
	static INPUTCORE_API const FKey F10;
	static INPUTCORE_API const FKey F11;
	static INPUTCORE_API const FKey F12;

	static INPUTCORE_API const FKey NumLock;

	static INPUTCORE_API const FKey ScrollLock;

	static INPUTCORE_API const FKey LeftShift;
	static INPUTCORE_API const FKey RightShift;
	static INPUTCORE_API const FKey LeftControl;
	static INPUTCORE_API const FKey RightControl;
	static INPUTCORE_API const FKey LeftAlt;
	static INPUTCORE_API const FKey RightAlt;
	static INPUTCORE_API const FKey LeftCommand;
	static INPUTCORE_API const FKey RightCommand;

	static INPUTCORE_API const FKey Semicolon;
	static INPUTCORE_API const FKey Equals;
	static INPUTCORE_API const FKey Comma;
	static INPUTCORE_API const FKey Underscore;
	static INPUTCORE_API const FKey Hyphen;
	static INPUTCORE_API const FKey Period;
	static INPUTCORE_API const FKey Slash;
	static INPUTCORE_API const FKey Tilde;
	static INPUTCORE_API const FKey LeftBracket;
	static INPUTCORE_API const FKey Backslash;
	static INPUTCORE_API const FKey RightBracket;
	static INPUTCORE_API const FKey Apostrophe;

	static INPUTCORE_API const FKey Ampersand;
	static INPUTCORE_API const FKey Asterix;
	static INPUTCORE_API const FKey Caret;
	static INPUTCORE_API const FKey Colon;
	static INPUTCORE_API const FKey Dollar;
	static INPUTCORE_API const FKey Exclamation;
	static INPUTCORE_API const FKey LeftParantheses;
	static INPUTCORE_API const FKey RightParantheses;
	static INPUTCORE_API const FKey Quote;

	static INPUTCORE_API const FKey A_AccentGrave;
	static INPUTCORE_API const FKey E_AccentGrave;
	static INPUTCORE_API const FKey E_AccentAigu;
	static INPUTCORE_API const FKey C_Cedille;
	static INPUTCORE_API const FKey Section;

	// Platform Keys
	// These keys platform specific versions of keys that go by different names.
	// The delete key is a good example, on Windows Delete is the virtual key for Delete.
	// On Macs, the Delete key is the virtual key for BackSpace.
	static INPUTCORE_API const FKey Platform_Delete;

	// Gamepad Keys
	static INPUTCORE_API const FKey Gamepad_Left2D;
	static INPUTCORE_API const FKey Gamepad_LeftX;
	static INPUTCORE_API const FKey Gamepad_LeftY;
	static INPUTCORE_API const FKey Gamepad_Right2D;
	static INPUTCORE_API const FKey Gamepad_RightX;
	static INPUTCORE_API const FKey Gamepad_RightY;
	static INPUTCORE_API const FKey Gamepad_LeftTriggerAxis;
	static INPUTCORE_API const FKey Gamepad_RightTriggerAxis;

	static INPUTCORE_API const FKey Gamepad_LeftThumbstick;
	static INPUTCORE_API const FKey Gamepad_RightThumbstick;
	static INPUTCORE_API const FKey Gamepad_Special_Left;
	static INPUTCORE_API const FKey Gamepad_Special_Left_X;
	static INPUTCORE_API const FKey Gamepad_Special_Left_Y;
	static INPUTCORE_API const FKey Gamepad_Special_Left_Touched;
	static INPUTCORE_API const FKey Gamepad_Special_Right;
	static INPUTCORE_API const FKey Gamepad_FaceButton_Bottom;
	static INPUTCORE_API const FKey Gamepad_FaceButton_Right;
	static INPUTCORE_API const FKey Gamepad_FaceButton_Left;
	static INPUTCORE_API const FKey Gamepad_FaceButton_Top;
	static INPUTCORE_API const FKey Gamepad_LeftShoulder;
	static INPUTCORE_API const FKey Gamepad_RightShoulder;
	static INPUTCORE_API const FKey Gamepad_LeftTrigger;
	static INPUTCORE_API const FKey Gamepad_RightTrigger;
	static INPUTCORE_API const FKey Gamepad_DPad_Up;
	static INPUTCORE_API const FKey Gamepad_DPad_Down;
	static INPUTCORE_API const FKey Gamepad_DPad_Right;
	static INPUTCORE_API const FKey Gamepad_DPad_Left;

	// Virtual key codes used for input axis button press/release emulation
	static INPUTCORE_API const FKey Gamepad_LeftStick_Up;
	static INPUTCORE_API const FKey Gamepad_LeftStick_Down;
	static INPUTCORE_API const FKey Gamepad_LeftStick_Right;
	static INPUTCORE_API const FKey Gamepad_LeftStick_Left;

	static INPUTCORE_API const FKey Gamepad_RightStick_Up;
	static INPUTCORE_API const FKey Gamepad_RightStick_Down;
	static INPUTCORE_API const FKey Gamepad_RightStick_Right;
	static INPUTCORE_API const FKey Gamepad_RightStick_Left;

	// static const FKey Vector axes (FVector; not float)
	static INPUTCORE_API const FKey Tilt;
	static INPUTCORE_API const FKey RotationRate;
	static INPUTCORE_API const FKey Gravity;
	static INPUTCORE_API const FKey Acceleration;

	// Gestures
	static INPUTCORE_API const FKey Gesture_Pinch;
	static INPUTCORE_API const FKey Gesture_Flick;
	static INPUTCORE_API const FKey Gesture_Rotate;
	static INPUTCORE_API const FKey Gesture_Pan;
	static INPUTCORE_API const FKey Gesture_Tap;
	static INPUTCORE_API const FKey Gesture_LongPress;

	// Steam Controller Specific
	static INPUTCORE_API const FKey Steam_Touch_0;
	static INPUTCORE_API const FKey Steam_Touch_1;
	static INPUTCORE_API const FKey Steam_Touch_2;
	static INPUTCORE_API const FKey Steam_Touch_3;
	static INPUTCORE_API const FKey Steam_Back_Left;
	static INPUTCORE_API const FKey Steam_Back_Right;

	// Xbox One global speech commands
	static INPUTCORE_API const FKey Global_Menu;
	static INPUTCORE_API const FKey Global_View;
	static INPUTCORE_API const FKey Global_Pause;
	static INPUTCORE_API const FKey Global_Play;
	static INPUTCORE_API const FKey Global_Back;

	// Android-specific
	static INPUTCORE_API const FKey Android_Back;
	static INPUTCORE_API const FKey Android_Volume_Up;
	static INPUTCORE_API const FKey Android_Volume_Down;
	static INPUTCORE_API const FKey Android_Menu;

	// HTC Vive Controller
	static INPUTCORE_API const FKey Vive_Left_Grip_Click;
	static INPUTCORE_API const FKey Vive_Left_Menu_Click;
	static INPUTCORE_API const FKey Vive_Left_Trigger_Click;
	static INPUTCORE_API const FKey Vive_Left_Trigger_Axis;
	static INPUTCORE_API const FKey Vive_Left_Trackpad_2D;
	static INPUTCORE_API const FKey Vive_Left_Trackpad_X;
	static INPUTCORE_API const FKey Vive_Left_Trackpad_Y;
	static INPUTCORE_API const FKey Vive_Left_Trackpad_Click;
	static INPUTCORE_API const FKey Vive_Left_Trackpad_Touch;
	static INPUTCORE_API const FKey Vive_Left_Trackpad_Up;
	static INPUTCORE_API const FKey Vive_Left_Trackpad_Down;
	static INPUTCORE_API const FKey Vive_Left_Trackpad_Left;
	static INPUTCORE_API const FKey Vive_Left_Trackpad_Right;
	static INPUTCORE_API const FKey Vive_Right_Grip_Click;
	static INPUTCORE_API const FKey Vive_Right_Menu_Click;
	static INPUTCORE_API const FKey Vive_Right_Trigger_Click;
	static INPUTCORE_API const FKey Vive_Right_Trigger_Axis;
	static INPUTCORE_API const FKey Vive_Right_Trackpad_2D;
	static INPUTCORE_API const FKey Vive_Right_Trackpad_X;
	static INPUTCORE_API const FKey Vive_Right_Trackpad_Y;
	static INPUTCORE_API const FKey Vive_Right_Trackpad_Click;
	static INPUTCORE_API const FKey Vive_Right_Trackpad_Touch;
	static INPUTCORE_API const FKey Vive_Right_Trackpad_Up;
	static INPUTCORE_API const FKey Vive_Right_Trackpad_Down;
	static INPUTCORE_API const FKey Vive_Right_Trackpad_Left;
	static INPUTCORE_API const FKey Vive_Right_Trackpad_Right;

	// Microsoft Mixed Reality Motion Controller
	static INPUTCORE_API const FKey MixedReality_Left_Menu_Click;
	static INPUTCORE_API const FKey MixedReality_Left_Grip_Click;
	static INPUTCORE_API const FKey MixedReality_Left_Trigger_Click;
	static INPUTCORE_API const FKey MixedReality_Left_Trigger_Axis;
	static INPUTCORE_API const FKey MixedReality_Left_Thumbstick_2D;
	static INPUTCORE_API const FKey MixedReality_Left_Thumbstick_X;
	static INPUTCORE_API const FKey MixedReality_Left_Thumbstick_Y;
	static INPUTCORE_API const FKey MixedReality_Left_Thumbstick_Click;
	static INPUTCORE_API const FKey MixedReality_Left_Thumbstick_Up;
	static INPUTCORE_API const FKey MixedReality_Left_Thumbstick_Down;
	static INPUTCORE_API const FKey MixedReality_Left_Thumbstick_Left;
	static INPUTCORE_API const FKey MixedReality_Left_Thumbstick_Right;
	static INPUTCORE_API const FKey MixedReality_Left_Trackpad_2D;
	static INPUTCORE_API const FKey MixedReality_Left_Trackpad_X;
	static INPUTCORE_API const FKey MixedReality_Left_Trackpad_Y;
	static INPUTCORE_API const FKey MixedReality_Left_Trackpad_Click;
	static INPUTCORE_API const FKey MixedReality_Left_Trackpad_Touch;
	static INPUTCORE_API const FKey MixedReality_Left_Trackpad_Up;
	static INPUTCORE_API const FKey MixedReality_Left_Trackpad_Down;
	static INPUTCORE_API const FKey MixedReality_Left_Trackpad_Left;
	static INPUTCORE_API const FKey MixedReality_Left_Trackpad_Right;
	static INPUTCORE_API const FKey MixedReality_Right_Menu_Click;
	static INPUTCORE_API const FKey MixedReality_Right_Grip_Click;
	static INPUTCORE_API const FKey MixedReality_Right_Trigger_Click;
	static INPUTCORE_API const FKey MixedReality_Right_Trigger_Axis;
	static INPUTCORE_API const FKey MixedReality_Right_Thumbstick_2D;
	static INPUTCORE_API const FKey MixedReality_Right_Thumbstick_X;
	static INPUTCORE_API const FKey MixedReality_Right_Thumbstick_Y;
	static INPUTCORE_API const FKey MixedReality_Right_Thumbstick_Click;
	static INPUTCORE_API const FKey MixedReality_Right_Thumbstick_Up;
	static INPUTCORE_API const FKey MixedReality_Right_Thumbstick_Down;
	static INPUTCORE_API const FKey MixedReality_Right_Thumbstick_Left;
	static INPUTCORE_API const FKey MixedReality_Right_Thumbstick_Right;
	static INPUTCORE_API const FKey MixedReality_Right_Trackpad_2D;
	static INPUTCORE_API const FKey MixedReality_Right_Trackpad_X;
	static INPUTCORE_API const FKey MixedReality_Right_Trackpad_Y;
	static INPUTCORE_API const FKey MixedReality_Right_Trackpad_Click;
	static INPUTCORE_API const FKey MixedReality_Right_Trackpad_Touch;
	static INPUTCORE_API const FKey MixedReality_Right_Trackpad_Up;
	static INPUTCORE_API const FKey MixedReality_Right_Trackpad_Down;
	static INPUTCORE_API const FKey MixedReality_Right_Trackpad_Left;
	static INPUTCORE_API const FKey MixedReality_Right_Trackpad_Right;

	// Oculus Touch Controller
	static INPUTCORE_API const FKey OculusTouch_Left_X_Click;
	static INPUTCORE_API const FKey OculusTouch_Left_Y_Click;
	static INPUTCORE_API const FKey OculusTouch_Left_X_Touch;
	static INPUTCORE_API const FKey OculusTouch_Left_Y_Touch;
	static INPUTCORE_API const FKey OculusTouch_Left_Menu_Click;
	static INPUTCORE_API const FKey OculusTouch_Left_Grip_Click;
	static INPUTCORE_API const FKey OculusTouch_Left_Grip_Axis;
	static INPUTCORE_API const FKey OculusTouch_Left_Trigger_Click;
	static INPUTCORE_API const FKey OculusTouch_Left_Trigger_Axis;
	static INPUTCORE_API const FKey OculusTouch_Left_Trigger_Touch;
	static INPUTCORE_API const FKey OculusTouch_Left_Thumbstick_2D;
	static INPUTCORE_API const FKey OculusTouch_Left_Thumbstick_X;
	static INPUTCORE_API const FKey OculusTouch_Left_Thumbstick_Y;
	static INPUTCORE_API const FKey OculusTouch_Left_Thumbstick_Click;
	static INPUTCORE_API const FKey OculusTouch_Left_Thumbstick_Touch;
	static INPUTCORE_API const FKey OculusTouch_Left_Thumbstick_Up;
	static INPUTCORE_API const FKey OculusTouch_Left_Thumbstick_Down;
	static INPUTCORE_API const FKey OculusTouch_Left_Thumbstick_Left;
	static INPUTCORE_API const FKey OculusTouch_Left_Thumbstick_Right;
	static INPUTCORE_API const FKey OculusTouch_Right_A_Click;
	static INPUTCORE_API const FKey OculusTouch_Right_B_Click;
	static INPUTCORE_API const FKey OculusTouch_Right_A_Touch;
	static INPUTCORE_API const FKey OculusTouch_Right_B_Touch;
	static INPUTCORE_API const FKey OculusTouch_Right_Grip_Click;
	static INPUTCORE_API const FKey OculusTouch_Right_Grip_Axis;
	static INPUTCORE_API const FKey OculusTouch_Right_Trigger_Click;
	static INPUTCORE_API const FKey OculusTouch_Right_Trigger_Axis;
	static INPUTCORE_API const FKey OculusTouch_Right_Trigger_Touch;
	static INPUTCORE_API const FKey OculusTouch_Right_Thumbstick_2D;
	static INPUTCORE_API const FKey OculusTouch_Right_Thumbstick_X;
	static INPUTCORE_API const FKey OculusTouch_Right_Thumbstick_Y;
	static INPUTCORE_API const FKey OculusTouch_Right_Thumbstick_Click;
	static INPUTCORE_API const FKey OculusTouch_Right_Thumbstick_Touch;
	static INPUTCORE_API const FKey OculusTouch_Right_Thumbstick_Up;
	static INPUTCORE_API const FKey OculusTouch_Right_Thumbstick_Down;
	static INPUTCORE_API const FKey OculusTouch_Right_Thumbstick_Left;
	static INPUTCORE_API const FKey OculusTouch_Right_Thumbstick_Right;

	// Valve Index Controller
	static INPUTCORE_API const FKey ValveIndex_Left_A_Click;
	static INPUTCORE_API const FKey ValveIndex_Left_B_Click;
	static INPUTCORE_API const FKey ValveIndex_Left_A_Touch;
	static INPUTCORE_API const FKey ValveIndex_Left_B_Touch;
	static INPUTCORE_API const FKey ValveIndex_Left_Grip_Axis;
	static INPUTCORE_API const FKey ValveIndex_Left_Grip_Force;
	static INPUTCORE_API const FKey ValveIndex_Left_Trigger_Click;
	static INPUTCORE_API const FKey ValveIndex_Left_Trigger_Axis;
	static INPUTCORE_API const FKey ValveIndex_Left_Trigger_Touch;
	static INPUTCORE_API const FKey ValveIndex_Left_Thumbstick_2D;
	static INPUTCORE_API const FKey ValveIndex_Left_Thumbstick_X;
	static INPUTCORE_API const FKey ValveIndex_Left_Thumbstick_Y;
	static INPUTCORE_API const FKey ValveIndex_Left_Thumbstick_Click;
	static INPUTCORE_API const FKey ValveIndex_Left_Thumbstick_Touch;
	static INPUTCORE_API const FKey ValveIndex_Left_Thumbstick_Up;
	static INPUTCORE_API const FKey ValveIndex_Left_Thumbstick_Down;
	static INPUTCORE_API const FKey ValveIndex_Left_Thumbstick_Left;
	static INPUTCORE_API const FKey ValveIndex_Left_Thumbstick_Right;
	static INPUTCORE_API const FKey ValveIndex_Left_Trackpad_2D;
	static INPUTCORE_API const FKey ValveIndex_Left_Trackpad_X;
	static INPUTCORE_API const FKey ValveIndex_Left_Trackpad_Y;
	static INPUTCORE_API const FKey ValveIndex_Left_Trackpad_Force;
	static INPUTCORE_API const FKey ValveIndex_Left_Trackpad_Touch;
	static INPUTCORE_API const FKey ValveIndex_Left_Trackpad_Up;
	static INPUTCORE_API const FKey ValveIndex_Left_Trackpad_Down;
	static INPUTCORE_API const FKey ValveIndex_Left_Trackpad_Left;
	static INPUTCORE_API const FKey ValveIndex_Left_Trackpad_Right;
	static INPUTCORE_API const FKey ValveIndex_Right_A_Click;
	static INPUTCORE_API const FKey ValveIndex_Right_B_Click;
	static INPUTCORE_API const FKey ValveIndex_Right_A_Touch;
	static INPUTCORE_API const FKey ValveIndex_Right_B_Touch;
	static INPUTCORE_API const FKey ValveIndex_Right_Grip_Axis;
	static INPUTCORE_API const FKey ValveIndex_Right_Grip_Force;
	static INPUTCORE_API const FKey ValveIndex_Right_Trigger_Click;
	static INPUTCORE_API const FKey ValveIndex_Right_Trigger_Axis;
	static INPUTCORE_API const FKey ValveIndex_Right_Trigger_Touch;
	static INPUTCORE_API const FKey ValveIndex_Right_Thumbstick_2D;
	static INPUTCORE_API const FKey ValveIndex_Right_Thumbstick_X;
	static INPUTCORE_API const FKey ValveIndex_Right_Thumbstick_Y;
	static INPUTCORE_API const FKey ValveIndex_Right_Thumbstick_Click;
	static INPUTCORE_API const FKey ValveIndex_Right_Thumbstick_Touch;
	static INPUTCORE_API const FKey ValveIndex_Right_Thumbstick_Up;
	static INPUTCORE_API const FKey ValveIndex_Right_Thumbstick_Down;
	static INPUTCORE_API const FKey ValveIndex_Right_Thumbstick_Left;
	static INPUTCORE_API const FKey ValveIndex_Right_Thumbstick_Right;
	static INPUTCORE_API const FKey ValveIndex_Right_Trackpad_2D;
	static INPUTCORE_API const FKey ValveIndex_Right_Trackpad_X;
	static INPUTCORE_API const FKey ValveIndex_Right_Trackpad_Y;
	static INPUTCORE_API const FKey ValveIndex_Right_Trackpad_Force;
	static INPUTCORE_API const FKey ValveIndex_Right_Trackpad_Touch;
	static INPUTCORE_API const FKey ValveIndex_Right_Trackpad_Up;
	static INPUTCORE_API const FKey ValveIndex_Right_Trackpad_Down;
	static INPUTCORE_API const FKey ValveIndex_Right_Trackpad_Left;
	static INPUTCORE_API const FKey ValveIndex_Right_Trackpad_Right;

	// Virtual buttons that use other buttons depending on the platform

	UE_DEPRECATED(5.7, "Use Virtual_Gamepad_Accept.GetVirtualKey() instead")
	static INPUTCORE_API const FKey Virtual_Accept;
	UE_DEPRECATED(5.7, "Use Virtual_Gamepad_Back.GetVirtualKey() instead")
	static INPUTCORE_API const FKey Virtual_Back;
	
	static INPUTCORE_API const FKey Virtual_Gamepad_Accept;
	static INPUTCORE_API const FKey Virtual_Gamepad_Back;

	static INPUTCORE_API const FKey Invalid;

	static const int32 NUM_TOUCH_KEYS = 11;
	static INPUTCORE_API const FKey TouchKeys[NUM_TOUCH_KEYS];

	// XR key names are parseable into exactly 4 tokens
	static const int32 NUM_XR_KEY_TOKENS = 4;

	static INPUTCORE_API EConsoleForGamepadLabels::Type ConsoleForGamepadLabels;

	static INPUTCORE_API const FName NAME_KeyboardCategory;
	static INPUTCORE_API const FName NAME_GamepadCategory;
	static INPUTCORE_API const FName NAME_MouseCategory;
	static INPUTCORE_API const FName NAME_VirtualCategory;

	static INPUTCORE_API void Initialize();
	static INPUTCORE_API void AddKey(const FKeyDetails& KeyDetails);
	// Map the two provided keys to the X and Y axes of the paired key
	static INPUTCORE_API void AddPairedKey(const FKeyDetails& PairedKeyDetails, FKey KeyX, FKey KeyY);
	static INPUTCORE_API void AddVirtualKey(const FKeyDetails& VirtualKeyDetails, const FKey& VirtualKeyValue);
	static INPUTCORE_API void GetAllKeys(TArray<FKey>& OutKeys);
	static INPUTCORE_API TSharedPtr<FKeyDetails> GetKeyDetails(const FKey Key);
	static INPUTCORE_API void RemoveKeysWithCategory(const FName InCategory);

	// These exist for backwards compatibility reasons only
	static bool IsModifierKey(FKey Key) { return Key.IsModifierKey(); }
	static bool IsGamepadKey(FKey Key) { return Key.IsGamepadKey(); }
	static bool IsAxis(FKey Key) { return Key.IsAxis1D(); }
	static bool IsBindableInBlueprints(const FKey Key) { return Key.IsBindableInBlueprints(); }
	static void SetConsoleForGamepadLabels(const EConsoleForGamepadLabels::Type Console) { ConsoleForGamepadLabels = Console; }

	// Function that provides remapping for some gamepad keys in display windows
	static INPUTCORE_API FText GetGamepadDisplayName(const FKey Key);

	static INPUTCORE_API void AddMenuCategoryDisplayInfo(const FName CategoryName, const FText DisplayName, const FName PaletteIcon);
	static INPUTCORE_API FText GetMenuCategoryDisplayName(const FName CategoryName);
	static INPUTCORE_API FName GetMenuCategoryPaletteIcon(const FName CategoryName);
	
	/**
	 * Stores some data which make up a pair from the "root" key of the pair.
	 *
	 * Created in EKeys::AddPairedKey. The key details of these types should always be valid
	 */
	struct FPairedKeyDetails
	{
		explicit FPairedKeyDetails(TSharedPtr<FKeyDetails> XDetails, TSharedPtr<FKeyDetails> YDetails)
			: XKeyDetails(XDetails)
			, YKeyDetails(YDetails)
		{ }
		
		TSharedPtr<FKeyDetails> XKeyDetails;
		TSharedPtr<FKeyDetails> YKeyDetails;
	};
	
	static INPUTCORE_API FPairedKeyDetails* GetPairedKeyDetails(const FKey& PairedKey);
	
private:

	struct FCategoryDisplayInfo
	{
		FText DisplayName;
		FName PaletteIcon;
	};

	static INPUTCORE_API TMap<FKey, TSharedPtr<FKeyDetails>> InputKeys;
	
	/**
	 * A map of Paired Key's (keys which are made up of a "pair" of other FKeys, such as Mouse2D
	 * being made up of MouseX and MouseY) to the keys which make up their pair.
	 */
	static INPUTCORE_API TMap<FKey, FPairedKeyDetails> PairedInputKeysLookup;
	
	static INPUTCORE_API TMap<FName, FCategoryDisplayInfo> MenuCategoryDisplayInfo;
	static INPUTCORE_API bool bInitialized;

};

/** Various states of touch inputs. */
UENUM()
namespace ETouchType
{
	enum Type : int
	{
		Began,
		Moved,
		Stationary,
		ForceChanged,
		FirstMove,
		Ended,

		NumTypes
	};
}


struct FInputKeyManager
{
public:
	static INPUTCORE_API FInputKeyManager& Get();

	INPUTCORE_API void GetCodesFromKey(const FKey Key, const uint32*& KeyCode, const uint32*& CharCode) const;

	/**
	 * Retrieves the key mapped to the specified character code.
	 * @param KeyCode	The key code to get the name for.
	 */
	INPUTCORE_API FKey GetKeyFromCodes( const uint32 KeyCode, const uint32 CharCode ) const;
	INPUTCORE_API void InitKeyMappings();
private:
	FInputKeyManager()
	{
		InitKeyMappings();
	}

	static TSharedPtr< FInputKeyManager > Instance;
	TMap<uint32, FKey> KeyMapVirtualToEnum;
	TMap<uint32, FKey> KeyMapCharToEnum;
};

UCLASS(abstract)
class UInputCoreTypes : public UObject
{
	GENERATED_BODY()

};
