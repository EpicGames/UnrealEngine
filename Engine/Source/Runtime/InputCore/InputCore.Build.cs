// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class InputCore : ModuleRules
{
	public InputCore(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PublicDependencyModuleNames.AddRange(new string[] { "Core", "CoreUObject" });

		if(Target.IsInPlatformGroup(UnrealPlatformGroup.IOS))
		{
			PrivateIncludePathModuleNames.Add("ApplicationCore");
		}
		if(Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PrivateIncludePathModuleNames.Add("SDL3");
		}
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
