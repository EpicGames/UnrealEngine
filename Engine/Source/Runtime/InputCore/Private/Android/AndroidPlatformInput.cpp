// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidPlatformInput.h"
#if USE_ANDROID_INPUT
#include <android/keycodes.h>
#endif
uint32 FAndroidPlatformInput::GetCharKeyMap(uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings)
{
#if USE_ANDROID_INPUT

#define ADDKEYMAP(KeyCode, KeyName)		if (NumMappings<MaxMappings) { KeyCodes[NumMappings]=KeyCode; if(KeyNames) { KeyNames[NumMappings]=KeyName; } ++NumMappings; };

	uint32 NumMappings = 0;

	if ( KeyCodes && (MaxMappings > 0) )
	{
		ADDKEYMAP( AKEYCODE_0, TEXT("Zero") );
		ADDKEYMAP( AKEYCODE_1, TEXT("One") );
		ADDKEYMAP( AKEYCODE_2, TEXT("Two") );
		ADDKEYMAP( AKEYCODE_3, TEXT("Three") );
		ADDKEYMAP( AKEYCODE_4, TEXT("Four") );
		ADDKEYMAP( AKEYCODE_5, TEXT("Five") );
		ADDKEYMAP( AKEYCODE_6, TEXT("Six") );
		ADDKEYMAP( AKEYCODE_7, TEXT("Seven") );
		ADDKEYMAP( AKEYCODE_8, TEXT("Eight") );
		ADDKEYMAP( AKEYCODE_9, TEXT("Nine") );

		ADDKEYMAP( AKEYCODE_A, TEXT("A") );
		ADDKEYMAP( AKEYCODE_B, TEXT("B") );
		ADDKEYMAP( AKEYCODE_C, TEXT("C") );
		ADDKEYMAP( AKEYCODE_D, TEXT("D") );
		ADDKEYMAP( AKEYCODE_E, TEXT("E") );
		ADDKEYMAP( AKEYCODE_F, TEXT("F") );
		ADDKEYMAP( AKEYCODE_G, TEXT("G") );
		ADDKEYMAP( AKEYCODE_H, TEXT("H") );
		ADDKEYMAP( AKEYCODE_I, TEXT("I") );
		ADDKEYMAP( AKEYCODE_J, TEXT("J") );
		ADDKEYMAP( AKEYCODE_K, TEXT("K") );
		ADDKEYMAP( AKEYCODE_L, TEXT("L") );
		ADDKEYMAP( AKEYCODE_M, TEXT("M") );
		ADDKEYMAP( AKEYCODE_N, TEXT("N") );
		ADDKEYMAP( AKEYCODE_O, TEXT("O") );
		ADDKEYMAP( AKEYCODE_P, TEXT("P") );
		ADDKEYMAP( AKEYCODE_Q, TEXT("Q") );
		ADDKEYMAP( AKEYCODE_R, TEXT("R") );
		ADDKEYMAP( AKEYCODE_S, TEXT("S") );
		ADDKEYMAP( AKEYCODE_T, TEXT("T") );
		ADDKEYMAP( AKEYCODE_U, TEXT("U") );
		ADDKEYMAP( AKEYCODE_V, TEXT("V") );
		ADDKEYMAP( AKEYCODE_W, TEXT("W") );
		ADDKEYMAP( AKEYCODE_X, TEXT("X") );
		ADDKEYMAP( AKEYCODE_Y, TEXT("Y") );
		ADDKEYMAP( AKEYCODE_Z, TEXT("Z") );

		ADDKEYMAP( AKEYCODE_SEMICOLON, TEXT("Semicolon") );
		ADDKEYMAP( AKEYCODE_EQUALS, TEXT("Equals") );
		ADDKEYMAP( AKEYCODE_COMMA, TEXT("Comma") );
		//ADDKEYMAP( '-', TEXT("Underscore") );
		ADDKEYMAP( AKEYCODE_PERIOD, TEXT("Period") );
		ADDKEYMAP( AKEYCODE_SLASH, TEXT("Slash") );
		ADDKEYMAP( AKEYCODE_GRAVE, TEXT("Tilde") );
		ADDKEYMAP( AKEYCODE_LEFT_BRACKET, TEXT("LeftBracket") );
		ADDKEYMAP( AKEYCODE_BACKSLASH, TEXT("Backslash") );
		ADDKEYMAP( AKEYCODE_RIGHT_BRACKET, TEXT("RightBracket") );
		ADDKEYMAP( AKEYCODE_APOSTROPHE, TEXT("Quote") );
		ADDKEYMAP( AKEYCODE_SPACE, TEXT("SpaceBar") );

		//ADDKEYMAP( VK_LBUTTON, TEXT("LeftMouseButton") );
		//ADDKEYMAP( VK_RBUTTON, TEXT("RightMouseButton") );
		//ADDKEYMAP( VK_MBUTTON, TEXT("MiddleMouseButton") );

		//ADDKEYMAP( AKEYCODE_BUTTON_THUMBL, TEXT("ThumbMouseButton") );
		//ADDKEYMAP( AKEYCODE_BUTTON_THUMBR, TEXT("ThumbMouseButton2") );

		ADDKEYMAP( AKEYCODE_DEL, TEXT("BackSpace") );
		ADDKEYMAP( AKEYCODE_TAB, TEXT("Tab") );
		ADDKEYMAP( AKEYCODE_ENTER, TEXT("Enter") );
		ADDKEYMAP( AKEYCODE_BREAK, TEXT("Pause") );

		//ADDKEYMAP( VK_CAPITAL, TEXT("CapsLock") );
		ADDKEYMAP( AKEYCODE_BACK, TEXT("Escape") );
		ADDKEYMAP( AKEYCODE_SPACE, TEXT("SpaceBar") );
		ADDKEYMAP( AKEYCODE_PAGE_UP, TEXT("PageUp") );
		ADDKEYMAP( AKEYCODE_PAGE_DOWN, TEXT("PageDown") );
		//ADDKEYMAP( VK_END, TEXT("End") );
		//ADDKEYMAP( VK_HOME, TEXT("Home") );

		//ADDKEYMAP( AKEYCODE_DPAD_LEFT, TEXT("Left") );
		//ADDKEYMAP( AKEYCODE_DPAD_UP, TEXT("Up") );
		//ADDKEYMAP( AKEYCODE_DPAD_RIGHT, TEXT("Right") );
		//ADDKEYMAP( AKEYCODE_DPAD_DOWN, TEXT("Down") );

		//ADDKEYMAP( VK_INSERT, TEXT("Insert") );
		ADDKEYMAP( AKEYCODE_FORWARD_DEL, TEXT("Delete") );

		ADDKEYMAP( AKEYCODE_STAR, TEXT("Multiply") );
		ADDKEYMAP( AKEYCODE_PLUS, TEXT("Add") );
		ADDKEYMAP( AKEYCODE_MINUS, TEXT("Subtract") );
		ADDKEYMAP( AKEYCODE_COMMA, TEXT("Decimal") );
		//ADDKEYMAP( AKEYCODE_SLASH, TEXT("Divide") );

		ADDKEYMAP( AKEYCODE_F1, TEXT("F1") );
		ADDKEYMAP( AKEYCODE_F2, TEXT("F2") );
		ADDKEYMAP( AKEYCODE_F3, TEXT("F3") );
		ADDKEYMAP( AKEYCODE_F4, TEXT("F4") );
		ADDKEYMAP( AKEYCODE_F5, TEXT("F5") );
		ADDKEYMAP( AKEYCODE_F6, TEXT("F6") );
		ADDKEYMAP( AKEYCODE_F7, TEXT("F7") );
		ADDKEYMAP( AKEYCODE_F8, TEXT("F8") );
		ADDKEYMAP( AKEYCODE_F9, TEXT("F9") );
		ADDKEYMAP( AKEYCODE_F10, TEXT("F10") );
		ADDKEYMAP( AKEYCODE_F11, TEXT("F11") );
		ADDKEYMAP( AKEYCODE_F12, TEXT("F12") );

		//ADDKEYMAP( AKEYCODE_NUM, TEXT("NumLock") );

		//ADDKEYMAP( VK_SCROLL, TEXT("ScrollLock") );

		ADDKEYMAP( AKEYCODE_SHIFT_LEFT, TEXT("LeftShift") );
		ADDKEYMAP( AKEYCODE_SHIFT_RIGHT, TEXT("RightShift") );
		ADDKEYMAP( AKEYCODE_CTRL_LEFT, TEXT("LeftControl") );
		ADDKEYMAP( AKEYCODE_CTRL_RIGHT, TEXT("RightControl") );
		ADDKEYMAP( AKEYCODE_ALT_LEFT, TEXT("LeftAlt") );
		ADDKEYMAP( AKEYCODE_ALT_RIGHT, TEXT("RightAlt") );
		ADDKEYMAP( AKEYCODE_META_LEFT, TEXT("LeftCommand") );
		ADDKEYMAP( AKEYCODE_META_RIGHT, TEXT("RightCommand") );
	}

	check(NumMappings < MaxMappings);
	return NumMappings;
#undef ADDKEYMAP

#else
	// by default use the standard printable keys for chars
	return FGenericPlatformInput::GetStandardPrintableKeyMap(KeyCodes, KeyNames, MaxMappings, true, true);
#endif
}

uint32 FAndroidPlatformInput::GetKeyMap( uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings )
{
#if USE_ANDROID_INPUT

#define ADDKEYMAP(KeyCode, KeyName)		if (NumMappings<MaxMappings) { KeyCodes[NumMappings]=KeyCode; if(KeyNames) { KeyNames[NumMappings]=KeyName; } ++NumMappings; };

	uint32 NumMappings = 0;

	if ( KeyCodes && (MaxMappings > 0) )
	{
		ADDKEYMAP( AKEYCODE_0, TEXT("Zero") );
		ADDKEYMAP( AKEYCODE_1, TEXT("One") );
		ADDKEYMAP( AKEYCODE_2, TEXT("Two") );
		ADDKEYMAP( AKEYCODE_3, TEXT("Three") );
		ADDKEYMAP( AKEYCODE_4, TEXT("Four") );
		ADDKEYMAP( AKEYCODE_5, TEXT("Five") );
		ADDKEYMAP( AKEYCODE_6, TEXT("Six") );
		ADDKEYMAP( AKEYCODE_7, TEXT("Seven") );
		ADDKEYMAP( AKEYCODE_8, TEXT("Eight") );
		ADDKEYMAP( AKEYCODE_9, TEXT("Nine") );

		ADDKEYMAP( AKEYCODE_A, TEXT("A") );
		ADDKEYMAP( AKEYCODE_B, TEXT("B") );
		ADDKEYMAP( AKEYCODE_C, TEXT("C") );
		ADDKEYMAP( AKEYCODE_D, TEXT("D") );
		ADDKEYMAP( AKEYCODE_E, TEXT("E") );
		ADDKEYMAP( AKEYCODE_F, TEXT("F") );
		ADDKEYMAP( AKEYCODE_G, TEXT("G") );
		ADDKEYMAP( AKEYCODE_H, TEXT("H") );
		ADDKEYMAP( AKEYCODE_I, TEXT("I") );
		ADDKEYMAP( AKEYCODE_J, TEXT("J") );
		ADDKEYMAP( AKEYCODE_K, TEXT("K") );
		ADDKEYMAP( AKEYCODE_L, TEXT("L") );
		ADDKEYMAP( AKEYCODE_M, TEXT("M") );
		ADDKEYMAP( AKEYCODE_N, TEXT("N") );
		ADDKEYMAP( AKEYCODE_O, TEXT("O") );
		ADDKEYMAP( AKEYCODE_P, TEXT("P") );
		ADDKEYMAP( AKEYCODE_Q, TEXT("Q") );
		ADDKEYMAP( AKEYCODE_R, TEXT("R") );
		ADDKEYMAP( AKEYCODE_S, TEXT("S") );
		ADDKEYMAP( AKEYCODE_T, TEXT("T") );
		ADDKEYMAP( AKEYCODE_U, TEXT("U") );
		ADDKEYMAP( AKEYCODE_V, TEXT("V") );
		ADDKEYMAP( AKEYCODE_W, TEXT("W") );
		ADDKEYMAP( AKEYCODE_X, TEXT("X") );
		ADDKEYMAP( AKEYCODE_Y, TEXT("Y") );
		ADDKEYMAP( AKEYCODE_Z, TEXT("Z") );

		ADDKEYMAP( AKEYCODE_SEMICOLON, TEXT("Semicolon") );
		ADDKEYMAP( AKEYCODE_EQUALS, TEXT("Equals") );
		ADDKEYMAP( AKEYCODE_COMMA, TEXT("Comma") );
		//ADDKEYMAP( '-', TEXT("Underscore") );
		ADDKEYMAP( AKEYCODE_PERIOD, TEXT("Period") );
		ADDKEYMAP( AKEYCODE_SLASH, TEXT("Slash") );
		ADDKEYMAP( AKEYCODE_GRAVE, TEXT("Tilde") );
		ADDKEYMAP( AKEYCODE_LEFT_BRACKET, TEXT("LeftBracket") );
		ADDKEYMAP( AKEYCODE_BACKSLASH, TEXT("Backslash") );
		ADDKEYMAP( AKEYCODE_RIGHT_BRACKET, TEXT("RightBracket") );
		ADDKEYMAP( AKEYCODE_APOSTROPHE, TEXT("Quote") );
		ADDKEYMAP( AKEYCODE_SPACE, TEXT("SpaceBar") );

		//ADDKEYMAP( VK_LBUTTON, TEXT("LeftMouseButton") );
		//ADDKEYMAP( VK_RBUTTON, TEXT("RightMouseButton") );
		//ADDKEYMAP( VK_MBUTTON, TEXT("MiddleMouseButton") );

		ADDKEYMAP( AKEYCODE_BUTTON_THUMBL, TEXT("ThumbMouseButton") );
		ADDKEYMAP( AKEYCODE_BUTTON_THUMBR, TEXT("ThumbMouseButton2") );

		ADDKEYMAP( AKEYCODE_DEL, TEXT("BackSpace") );
		ADDKEYMAP( AKEYCODE_TAB, TEXT("Tab") );
		ADDKEYMAP( AKEYCODE_ENTER, TEXT("Enter") );
		//ADDKEYMAP( VK_PAUSE, TEXT("Pause") );

		//ADDKEYMAP( VK_CAPITAL, TEXT("CapsLock") );
		ADDKEYMAP( AKEYCODE_BACK, TEXT("Escape") );
		ADDKEYMAP( AKEYCODE_SPACE, TEXT("SpaceBar") );
		ADDKEYMAP( AKEYCODE_PAGE_UP, TEXT("PageUp") );
		ADDKEYMAP( AKEYCODE_PAGE_DOWN, TEXT("PageDown") );
		//ADDKEYMAP( VK_END, TEXT("End") );
		//ADDKEYMAP( VK_HOME, TEXT("Home") );

		ADDKEYMAP( AKEYCODE_DPAD_LEFT, TEXT("Left") );
		ADDKEYMAP( AKEYCODE_DPAD_UP, TEXT("Up") );
		ADDKEYMAP( AKEYCODE_DPAD_RIGHT, TEXT("Right") );
		ADDKEYMAP( AKEYCODE_DPAD_DOWN, TEXT("Down") );

		//ADDKEYMAP( VK_INSERT, TEXT("Insert") );
		//ADDKEYMAP( AKEYCODE_DEL, TEXT("Delete") );

		ADDKEYMAP( AKEYCODE_AT, TEXT("At"));
		ADDKEYMAP( AKEYCODE_POUND, TEXT("Pound"));

		ADDKEYMAP( AKEYCODE_STAR, TEXT("Multiply") );
		ADDKEYMAP( AKEYCODE_PLUS, TEXT("Add") );
		ADDKEYMAP( AKEYCODE_MINUS, TEXT("Subtract") );
		ADDKEYMAP( AKEYCODE_COMMA, TEXT("Decimal") );
		//ADDKEYMAP( AKEYCODE_SLASH, TEXT("Divide") );

		//ADDKEYMAP( VK_F1, TEXT("F1") );
		//ADDKEYMAP( VK_F2, TEXT("F2") );
		//ADDKEYMAP( VK_F3, TEXT("F3") );
		//ADDKEYMAP( VK_F4, TEXT("F4") );
		//ADDKEYMAP( VK_F5, TEXT("F5") );
		//ADDKEYMAP( VK_F6, TEXT("F6") );
		//ADDKEYMAP( VK_F7, TEXT("F7") );
		//ADDKEYMAP( VK_F8, TEXT("F8") );
		//ADDKEYMAP( VK_F9, TEXT("F9") );
		//ADDKEYMAP( VK_F10, TEXT("F10") );
		//ADDKEYMAP( VK_F11, TEXT("F11") );
		//ADDKEYMAP( VK_F12, TEXT("F12") );

		ADDKEYMAP( AKEYCODE_NUM, TEXT("NumLock") );

		//ADDKEYMAP( VK_SCROLL, TEXT("ScrollLock") );

		ADDKEYMAP( AKEYCODE_SHIFT_LEFT, TEXT("LeftShift") );
		ADDKEYMAP( AKEYCODE_SHIFT_RIGHT, TEXT("RightShift") );
		ADDKEYMAP( AKEYCODE_CTRL_LEFT, TEXT("LeftControl") );
		ADDKEYMAP( AKEYCODE_CTRL_RIGHT, TEXT("RightControl") );
		ADDKEYMAP( AKEYCODE_ALT_LEFT, TEXT("LeftAlt") );
		ADDKEYMAP( AKEYCODE_ALT_RIGHT, TEXT("RightAlt") );
		//ADDKEYMAP( VK_LWIN, TEXT("LeftCommand") );
		//ADDKEYMAP( VK_RWIN, TEXT("RightCommand") );

		ADDKEYMAP(AKEYCODE_BACK, TEXT("Android_Back"));
		ADDKEYMAP(AKEYCODE_VOLUME_UP, TEXT("Android_Volume_Up"));
		ADDKEYMAP(AKEYCODE_VOLUME_DOWN, TEXT("Android_Volume_Down"));
		ADDKEYMAP(AKEYCODE_MENU, TEXT("Android_Menu"));

	}

	check(NumMappings < MaxMappings);
	return NumMappings;
#undef ADDKEYMAP

#else
	// nothing we can do by default here - sub-platforms may need to do stuff here
	return 0;
#endif
}
