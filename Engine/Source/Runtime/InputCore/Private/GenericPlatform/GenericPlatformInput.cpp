// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/GenericPlatformInput.h"

uint32 FGenericPlatformInput::GetStandardPrintableKeyMap(uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings, bool bMapUppercaseKeys, bool bMapLowercaseKeys)
{
	uint32 NumMappings = 0;

#define ADDKEYMAP(KeyCode, KeyName)		if (NumMappings<MaxMappings) { KeyCodes[NumMappings]=KeyCode; KeyNames[NumMappings]=KeyName; ++NumMappings; };

	ADDKEYMAP( '0', TEXT("Zero") );
	ADDKEYMAP( '1', TEXT("One") );
	ADDKEYMAP( '2', TEXT("Two") );
	ADDKEYMAP( '3', TEXT("Three") );
	ADDKEYMAP( '4', TEXT("Four") );
	ADDKEYMAP( '5', TEXT("Five") );
	ADDKEYMAP( '6', TEXT("Six") );
	ADDKEYMAP( '7', TEXT("Seven") );
	ADDKEYMAP( '8', TEXT("Eight") );
	ADDKEYMAP( '9', TEXT("Nine") );

	// we map both upper and lower
	if (bMapUppercaseKeys)
	{
		ADDKEYMAP( 'A', TEXT("A") );
		ADDKEYMAP( 'B', TEXT("B") );
		ADDKEYMAP( 'C', TEXT("C") );
		ADDKEYMAP( 'D', TEXT("D") );
		ADDKEYMAP( 'E', TEXT("E") );
		ADDKEYMAP( 'F', TEXT("F") );
		ADDKEYMAP( 'G', TEXT("G") );
		ADDKEYMAP( 'H', TEXT("H") );
		ADDKEYMAP( 'I', TEXT("I") );
		ADDKEYMAP( 'J', TEXT("J") );
		ADDKEYMAP( 'K', TEXT("K") );
		ADDKEYMAP( 'L', TEXT("L") );
		ADDKEYMAP( 'M', TEXT("M") );
		ADDKEYMAP( 'N', TEXT("N") );
		ADDKEYMAP( 'O', TEXT("O") );
		ADDKEYMAP( 'P', TEXT("P") );
		ADDKEYMAP( 'Q', TEXT("Q") );
		ADDKEYMAP( 'R', TEXT("R") );
		ADDKEYMAP( 'S', TEXT("S") );
		ADDKEYMAP( 'T', TEXT("T") );
		ADDKEYMAP( 'U', TEXT("U") );
		ADDKEYMAP( 'V', TEXT("V") );
		ADDKEYMAP( 'W', TEXT("W") );
		ADDKEYMAP( 'X', TEXT("X") );
		ADDKEYMAP( 'Y', TEXT("Y") );
		ADDKEYMAP( 'Z', TEXT("Z") );
	}

	if (bMapLowercaseKeys)
	{
		ADDKEYMAP( 'a', TEXT("A") );
		ADDKEYMAP( 'b', TEXT("B") );
		ADDKEYMAP( 'c', TEXT("C") );
		ADDKEYMAP( 'd', TEXT("D") );
		ADDKEYMAP( 'e', TEXT("E") );
		ADDKEYMAP( 'f', TEXT("F") );
		ADDKEYMAP( 'g', TEXT("G") );
		ADDKEYMAP( 'h', TEXT("H") );
		ADDKEYMAP( 'i', TEXT("I") );
		ADDKEYMAP( 'j', TEXT("J") );
		ADDKEYMAP( 'k', TEXT("K") );
		ADDKEYMAP( 'l', TEXT("L") );
		ADDKEYMAP( 'm', TEXT("M") );
		ADDKEYMAP( 'n', TEXT("N") );
		ADDKEYMAP( 'o', TEXT("O") );
		ADDKEYMAP( 'p', TEXT("P") );
		ADDKEYMAP( 'q', TEXT("Q") );
		ADDKEYMAP( 'r', TEXT("R") );
		ADDKEYMAP( 's', TEXT("S") );
		ADDKEYMAP( 't', TEXT("T") );
		ADDKEYMAP( 'u', TEXT("U") );
		ADDKEYMAP( 'v', TEXT("V") );
		ADDKEYMAP( 'w', TEXT("W") );
		ADDKEYMAP( 'x', TEXT("X") );
		ADDKEYMAP( 'y', TEXT("Y") );
		ADDKEYMAP( 'z', TEXT("Z") );
	}

	ADDKEYMAP( ';', TEXT("Semicolon") );
	ADDKEYMAP( '=', TEXT("Equals") );
	ADDKEYMAP( ',', TEXT("Comma") );
	ADDKEYMAP( '-', TEXT("Hyphen") );
	ADDKEYMAP( '.', TEXT("Period") );
	ADDKEYMAP( '/', TEXT("Slash") );
	ADDKEYMAP( '`', TEXT("Tilde") );
	ADDKEYMAP( '[', TEXT("LeftBracket") );
	ADDKEYMAP( '\\', TEXT("Backslash") );
	ADDKEYMAP( ']', TEXT("RightBracket") );
	ADDKEYMAP( '\'', TEXT("Apostrophe") );
	ADDKEYMAP(' ', TEXT("SpaceBar"));

	// AZERTY Keys
	ADDKEYMAP( '&', TEXT("Ampersand") );
	ADDKEYMAP( '*', TEXT("Asterix") );
	ADDKEYMAP( '^', TEXT("Caret") );
	ADDKEYMAP( ':', TEXT("Colon") );
	ADDKEYMAP( '$', TEXT("Dollar") );
	ADDKEYMAP( '!', TEXT("Exclamation") );
	ADDKEYMAP( '(', TEXT("LeftParantheses") );
	ADDKEYMAP( ')', TEXT("RightParantheses") );
	ADDKEYMAP( '"', TEXT("Quote") );
	ADDKEYMAP( '_', TEXT("Underscore") );
	ADDKEYMAP( 224, TEXT("A_AccentGrave") );
	ADDKEYMAP( 231, TEXT("C_Cedille") );
	ADDKEYMAP( 233, TEXT("E_AccentAigu") );
	ADDKEYMAP( 232, TEXT("E_AccentGrave") );
	ADDKEYMAP( 167, TEXT("Section") );

#undef ADDKEYMAP

	return NumMappings;
}

