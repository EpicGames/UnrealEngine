// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOS/IOSPlatformInput.h"
#include "IOS/IOSInputInterface.h"
#include "IOS/IOSInputDefinitions.h"

uint32 FIOSPlatformInput::GetKeyMap( uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings )
{
#define ADDKEYMAP(KeyCode, KeyName)		if (NumMappings<MaxMappings) { KeyCodes[NumMappings]=static_cast<uint32>(KeyCode); KeyNames[NumMappings]=KeyName; ++NumMappings; };
	
	uint32 NumMappings = 0;
	
	using namespace IOS;
	
	if (KeyCodes && KeyNames && (MaxMappings > 0))
	{
		ADDKEYMAP(EUnrealKeyCode::Enter, TEXT("Enter"));
		ADDKEYMAP(EUnrealKeyCode::Backspace, TEXT("BackSpace"));
		ADDKEYMAP(EUnrealKeyCode::Escape, TEXT("Escape"));
		ADDKEYMAP(EUnrealKeyCode::SpaceBar, TEXT("SpaceBar"));
		ADDKEYMAP(EUnrealKeyCode::Tab, TEXT("Tab"));
		
		ADDKEYMAP(EUnrealKeyCode::Left, TEXT("Left"));
		ADDKEYMAP(EUnrealKeyCode::Right, TEXT("Right"));
		ADDKEYMAP(EUnrealKeyCode::Down, TEXT("Down"));
		ADDKEYMAP(EUnrealKeyCode::Up, TEXT("Up"));
		
		ADDKEYMAP(EUnrealKeyCode::LeftControl, TEXT("LeftControl"));
		ADDKEYMAP(EUnrealKeyCode::LeftShift, TEXT("LeftShift"));
		ADDKEYMAP(EUnrealKeyCode::LeftAlt, TEXT("LeftAlt"));
		ADDKEYMAP(EUnrealKeyCode::LeftCommand, TEXT("LeftCommand"));
		ADDKEYMAP(EUnrealKeyCode::CapsLock, TEXT("CapsLock"));
		ADDKEYMAP(EUnrealKeyCode::RightControl, TEXT("RightControl"));
		ADDKEYMAP(EUnrealKeyCode::RightShift, TEXT("RightShift"));
		ADDKEYMAP(EUnrealKeyCode::RightAlt, TEXT("RightAlt"));
		ADDKEYMAP(EUnrealKeyCode::RightCommand, TEXT("RightCommand"));
		
		ADDKEYMAP(EUnrealKeyCode::F1, TEXT("F1"));
		ADDKEYMAP(EUnrealKeyCode::F2, TEXT("F2"));
		ADDKEYMAP(EUnrealKeyCode::F3, TEXT("F3"));
		ADDKEYMAP(EUnrealKeyCode::F4, TEXT("F4"));
		ADDKEYMAP(EUnrealKeyCode::F5, TEXT("F5"));
		ADDKEYMAP(EUnrealKeyCode::F6, TEXT("F6"));
		ADDKEYMAP(EUnrealKeyCode::F7, TEXT("F7"));
		ADDKEYMAP(EUnrealKeyCode::F8, TEXT("F8"));
		ADDKEYMAP(EUnrealKeyCode::F9, TEXT("F9"));
		ADDKEYMAP(EUnrealKeyCode::F10, TEXT("F10"));
		ADDKEYMAP(EUnrealKeyCode::F11, TEXT("F11"));
		ADDKEYMAP(EUnrealKeyCode::F12, TEXT("F12"));
		
		ADDKEYMAP(EUnrealKeyCode::Pause, TEXT("Pause") );
		ADDKEYMAP(EUnrealKeyCode::NumLock, TEXT("NumLock") );
		ADDKEYMAP(EUnrealKeyCode::ScrollLock, TEXT("ScrollLock") );

		ADDKEYMAP(EUnrealKeyCode::PageUp, TEXT("PageUp") );
		ADDKEYMAP(EUnrealKeyCode::PageDown, TEXT("PageDown") );
		ADDKEYMAP(EUnrealKeyCode::End, TEXT("End") );
		ADDKEYMAP(EUnrealKeyCode::Home, TEXT("Home") );
		ADDKEYMAP(EUnrealKeyCode::Delete, TEXT("Delete") );
		ADDKEYMAP(EUnrealKeyCode::Insert, TEXT("Insert") );

		ADDKEYMAP(EUnrealKeyCode::NumPad0, TEXT("NumPadZero") );
		ADDKEYMAP(EUnrealKeyCode::NumPad1, TEXT("NumPadOne") );
		ADDKEYMAP(EUnrealKeyCode::NumPad2, TEXT("NumPadTwo") );
		ADDKEYMAP(EUnrealKeyCode::NumPad3, TEXT("NumPadThree") );
		ADDKEYMAP(EUnrealKeyCode::NumPad4, TEXT("NumPadFour") );
		ADDKEYMAP(EUnrealKeyCode::NumPad5, TEXT("NumPadFive") );
		ADDKEYMAP(EUnrealKeyCode::NumPad6, TEXT("NumPadSix") );
		ADDKEYMAP(EUnrealKeyCode::NumPad7, TEXT("NumPadSeven") );
		ADDKEYMAP(EUnrealKeyCode::NumPad8, TEXT("NumPadEight") );
		ADDKEYMAP(EUnrealKeyCode::NumPad9, TEXT("NumPadNine") );

		ADDKEYMAP(EUnrealKeyCode::Multiply, TEXT("Multiply") );
		ADDKEYMAP(EUnrealKeyCode::Add, TEXT("Add") );
		ADDKEYMAP(EUnrealKeyCode::Subtract, TEXT("Subtract") );
		ADDKEYMAP(EUnrealKeyCode::Decimal, TEXT("Decimal") );
		ADDKEYMAP(EUnrealKeyCode::Divide, TEXT("Divide") );
	}
	return NumMappings;

#undef ADDKEYMAP
}

uint32 FIOSPlatformInput::GetCharKeyMap(uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings)
{
	return FGenericPlatformInput::GetStandardPrintableKeyMap(KeyCodes, KeyNames, MaxMappings, true, true);
}
