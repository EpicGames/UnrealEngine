// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputCoreTypes.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyPortFlags.h"
#include "HAL/PlatformInput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputCoreTypes)

DEFINE_LOG_CATEGORY(LogInput);

LLM_DEFINE_TAG(InputCore_EKeys);
LLM_DEFINE_TAG(InputCore_KeyManager);

#define LOCTEXT_NAMESPACE "InputKeys"

const TCHAR* FKey::SyntheticCharPrefix = TEXT("UnknownCharCode_");

const FKey EKeys::AnyKey("AnyKey");

const FKey EKeys::MouseX("MouseX");
const FKey EKeys::MouseY("MouseY");
const FKey EKeys::Mouse2D("Mouse2D");
const FKey EKeys::MouseScrollUp("MouseScrollUp");
const FKey EKeys::MouseScrollDown("MouseScrollDown");
const FKey EKeys::MouseWheelAxis("MouseWheelAxis");

const FKey EKeys::LeftMouseButton("LeftMouseButton");
const FKey EKeys::RightMouseButton("RightMouseButton");
const FKey EKeys::MiddleMouseButton("MiddleMouseButton");
const FKey EKeys::ThumbMouseButton("ThumbMouseButton");
const FKey EKeys::ThumbMouseButton2("ThumbMouseButton2");

const FKey EKeys::BackSpace("BackSpace");
const FKey EKeys::Tab("Tab");
const FKey EKeys::Enter("Enter");
const FKey EKeys::Pause("Pause");

const FKey EKeys::CapsLock("CapsLock");
const FKey EKeys::Escape("Escape");
const FKey EKeys::SpaceBar("SpaceBar");
const FKey EKeys::PageUp("PageUp");
const FKey EKeys::PageDown("PageDown");
const FKey EKeys::End("End");
const FKey EKeys::Home("Home");

const FKey EKeys::Left("Left");
const FKey EKeys::Up("Up");
const FKey EKeys::Right("Right");
const FKey EKeys::Down("Down");

const FKey EKeys::Insert("Insert");
const FKey EKeys::Delete("Delete");

const FKey EKeys::Zero("Zero");
const FKey EKeys::One("One");
const FKey EKeys::Two("Two");
const FKey EKeys::Three("Three");
const FKey EKeys::Four("Four");
const FKey EKeys::Five("Five");
const FKey EKeys::Six("Six");
const FKey EKeys::Seven("Seven");
const FKey EKeys::Eight("Eight");
const FKey EKeys::Nine("Nine");

const FKey EKeys::A("A");
const FKey EKeys::B("B");
const FKey EKeys::C("C");
const FKey EKeys::D("D");
const FKey EKeys::E("E");
const FKey EKeys::F("F");
const FKey EKeys::G("G");
const FKey EKeys::H("H");
const FKey EKeys::I("I");
const FKey EKeys::J("J");
const FKey EKeys::K("K");
const FKey EKeys::L("L");
const FKey EKeys::M("M");
const FKey EKeys::N("N");
const FKey EKeys::O("O");
const FKey EKeys::P("P");
const FKey EKeys::Q("Q");
const FKey EKeys::R("R");
const FKey EKeys::S("S");
const FKey EKeys::T("T");
const FKey EKeys::U("U");
const FKey EKeys::V("V");
const FKey EKeys::W("W");
const FKey EKeys::X("X");
const FKey EKeys::Y("Y");
const FKey EKeys::Z("Z");

const FKey EKeys::NumPadZero("NumPadZero");
const FKey EKeys::NumPadOne("NumPadOne");
const FKey EKeys::NumPadTwo("NumPadTwo");
const FKey EKeys::NumPadThree("NumPadThree");
const FKey EKeys::NumPadFour("NumPadFour");
const FKey EKeys::NumPadFive("NumPadFive");
const FKey EKeys::NumPadSix("NumPadSix");
const FKey EKeys::NumPadSeven("NumPadSeven");
const FKey EKeys::NumPadEight("NumPadEight");
const FKey EKeys::NumPadNine("NumPadNine");

const FKey EKeys::Multiply("Multiply");
const FKey EKeys::Add("Add");
const FKey EKeys::Subtract("Subtract");
const FKey EKeys::Decimal("Decimal");
const FKey EKeys::Divide("Divide");

const FKey EKeys::F1("F1");
const FKey EKeys::F2("F2");
const FKey EKeys::F3("F3");
const FKey EKeys::F4("F4");
const FKey EKeys::F5("F5");
const FKey EKeys::F6("F6");
const FKey EKeys::F7("F7");
const FKey EKeys::F8("F8");
const FKey EKeys::F9("F9");
const FKey EKeys::F10("F10");
const FKey EKeys::F11("F11");
const FKey EKeys::F12("F12");

const FKey EKeys::NumLock("NumLock");

const FKey EKeys::ScrollLock("ScrollLock");

const FKey EKeys::LeftShift("LeftShift");
const FKey EKeys::RightShift("RightShift");
const FKey EKeys::LeftControl("LeftControl");
const FKey EKeys::RightControl("RightControl");
const FKey EKeys::LeftAlt("LeftAlt");
const FKey EKeys::RightAlt("RightAlt");
const FKey EKeys::LeftCommand("LeftCommand");
const FKey EKeys::RightCommand("RightCommand");

const FKey EKeys::Semicolon("Semicolon");
const FKey EKeys::Equals("Equals");
const FKey EKeys::Comma("Comma");
const FKey EKeys::Underscore("Underscore");
const FKey EKeys::Hyphen("Hyphen");
const FKey EKeys::Period("Period");
const FKey EKeys::Slash("Slash");
const FKey EKeys::Tilde("Tilde");
const FKey EKeys::LeftBracket("LeftBracket");
const FKey EKeys::LeftParantheses("LeftParantheses");
const FKey EKeys::Backslash("Backslash");
const FKey EKeys::RightBracket("RightBracket");
const FKey EKeys::RightParantheses("RightParantheses");
const FKey EKeys::Apostrophe("Apostrophe");
const FKey EKeys::Quote("Quote");

const FKey EKeys::Asterix("Asterix");
const FKey EKeys::Ampersand("Ampersand");
const FKey EKeys::Caret("Caret");
const FKey EKeys::Dollar("Dollar");
const FKey EKeys::Exclamation("Exclamation");
const FKey EKeys::Colon("Colon");

const FKey EKeys::A_AccentGrave("A_AccentGrave");
const FKey EKeys::E_AccentGrave("E_AccentGrave");
const FKey EKeys::E_AccentAigu("E_AccentAigu");
const FKey EKeys::C_Cedille("C_Cedille");
const FKey EKeys::Section("Section");

// Setup platform specific keys
const FKey EKeys::Platform_Delete = FPlatformInput::GetPlatformDeleteKey();

// Ensure that the Gamepad_ names match those in GenericApplication.cpp
const FKey EKeys::Gamepad_Left2D("Gamepad_Left2D");
const FKey EKeys::Gamepad_LeftX("Gamepad_LeftX");
const FKey EKeys::Gamepad_LeftY("Gamepad_LeftY");
const FKey EKeys::Gamepad_Right2D("Gamepad_Right2D");
const FKey EKeys::Gamepad_RightX("Gamepad_RightX");
const FKey EKeys::Gamepad_RightY("Gamepad_RightY");
const FKey EKeys::Gamepad_LeftTriggerAxis("Gamepad_LeftTriggerAxis");
const FKey EKeys::Gamepad_RightTriggerAxis("Gamepad_RightTriggerAxis");

const FKey EKeys::Gamepad_LeftThumbstick("Gamepad_LeftThumbstick");
const FKey EKeys::Gamepad_RightThumbstick("Gamepad_RightThumbstick");
const FKey EKeys::Gamepad_Special_Left("Gamepad_Special_Left");
const FKey EKeys::Gamepad_Special_Left_X("Gamepad_Special_Left_X");
const FKey EKeys::Gamepad_Special_Left_Y("Gamepad_Special_Left_Y");
const FKey EKeys::Gamepad_Special_Left_Touched("Gamepad_Special_Left_Touched");
const FKey EKeys::Gamepad_Special_Right("Gamepad_Special_Right");
const FKey EKeys::Gamepad_FaceButton_Bottom("Gamepad_FaceButton_Bottom");
const FKey EKeys::Gamepad_FaceButton_Right("Gamepad_FaceButton_Right");
const FKey EKeys::Gamepad_FaceButton_Left("Gamepad_FaceButton_Left");
const FKey EKeys::Gamepad_FaceButton_Top("Gamepad_FaceButton_Top");
const FKey EKeys::Gamepad_LeftShoulder("Gamepad_LeftShoulder");
const FKey EKeys::Gamepad_RightShoulder("Gamepad_RightShoulder");
const FKey EKeys::Gamepad_LeftTrigger("Gamepad_LeftTrigger");
const FKey EKeys::Gamepad_RightTrigger("Gamepad_RightTrigger");
const FKey EKeys::Gamepad_DPad_Up("Gamepad_DPad_Up");
const FKey EKeys::Gamepad_DPad_Down("Gamepad_DPad_Down");
const FKey EKeys::Gamepad_DPad_Right("Gamepad_DPad_Right");
const FKey EKeys::Gamepad_DPad_Left("Gamepad_DPad_Left");

// Virtual key codes used for input axis button press/release emulation
const FKey EKeys::Gamepad_LeftStick_Up("Gamepad_LeftStick_Up");
const FKey EKeys::Gamepad_LeftStick_Down("Gamepad_LeftStick_Down");
const FKey EKeys::Gamepad_LeftStick_Right("Gamepad_LeftStick_Right");
const FKey EKeys::Gamepad_LeftStick_Left("Gamepad_LeftStick_Left");

const FKey EKeys::Gamepad_RightStick_Up("Gamepad_RightStick_Up");
const FKey EKeys::Gamepad_RightStick_Down("Gamepad_RightStick_Down");
const FKey EKeys::Gamepad_RightStick_Right("Gamepad_RightStick_Right");
const FKey EKeys::Gamepad_RightStick_Left("Gamepad_RightStick_Left");

// const FKey EKeys::Vector axes (FVector("Vector axes (FVector"); not float)
const FKey EKeys::Tilt("Tilt");
const FKey EKeys::RotationRate("RotationRate");
const FKey EKeys::Gravity("Gravity");
const FKey EKeys::Acceleration("Acceleration");

// Fingers
const FKey EKeys::TouchKeys[NUM_TOUCH_KEYS];

// Gestures
const FKey EKeys::Gesture_Pinch("Gesture_Pinch");
const FKey EKeys::Gesture_Flick("Gesture_Flick");
const FKey EKeys::Gesture_Rotate("Gesture_Rotate");
const FKey EKeys::Gesture_Pan("Gesture_Pan");
const FKey EKeys::Gesture_Tap("Gesture_Tap");
const FKey EKeys::Gesture_LongPress("Gesture_LongPress");

// Steam Controller Specific
const FKey EKeys::Steam_Touch_0("Steam_Touch_0");
const FKey EKeys::Steam_Touch_1("Steam_Touch_1");
const FKey EKeys::Steam_Touch_2("Steam_Touch_2");
const FKey EKeys::Steam_Touch_3("Steam_Touch_3");
const FKey EKeys::Steam_Back_Left("Steam_Back_Left");
const FKey EKeys::Steam_Back_Right("Steam_Back_Right");

// Xbox One global speech commands
const FKey EKeys::Global_Menu("Global_Menu");
const FKey EKeys::Global_View("Global_View");
const FKey EKeys::Global_Pause("Global_Pause");
const FKey EKeys::Global_Play("Global_Play");
const FKey EKeys::Global_Back("Global_Back");

// Android-specific
const FKey EKeys::Android_Back("Android_Back");
const FKey EKeys::Android_Volume_Up("Android_Volume_Up");
const FKey EKeys::Android_Volume_Down("Android_Volume_Down");
const FKey EKeys::Android_Menu("Android_Menu");

// HTC Vive Controller
const FKey EKeys::Vive_Left_Grip_Click("Vive_Left_Grip_Click");
const FKey EKeys::Vive_Left_Menu_Click("Vive_Left_Menu_Click");
const FKey EKeys::Vive_Left_Trigger_Click("Vive_Left_Trigger_Click");
const FKey EKeys::Vive_Left_Trigger_Axis("Vive_Left_Trigger_Axis");
const FKey EKeys::Vive_Left_Trackpad_2D("Vive_Left_Trackpad_2D");
const FKey EKeys::Vive_Left_Trackpad_X("Vive_Left_Trackpad_X");
const FKey EKeys::Vive_Left_Trackpad_Y("Vive_Left_Trackpad_Y");
const FKey EKeys::Vive_Left_Trackpad_Click("Vive_Left_Trackpad_Click");
const FKey EKeys::Vive_Left_Trackpad_Touch("Vive_Left_Trackpad_Touch");
const FKey EKeys::Vive_Left_Trackpad_Up("Vive_Left_Trackpad_Up");
const FKey EKeys::Vive_Left_Trackpad_Down("Vive_Left_Trackpad_Down");
const FKey EKeys::Vive_Left_Trackpad_Left("Vive_Left_Trackpad_Left");
const FKey EKeys::Vive_Left_Trackpad_Right("Vive_Left_Trackpad_Right");
const FKey EKeys::Vive_Right_Grip_Click("Vive_Right_Grip_Click");
const FKey EKeys::Vive_Right_Menu_Click("Vive_Right_Menu_Click");
const FKey EKeys::Vive_Right_Trigger_Click("Vive_Right_Trigger_Click");
const FKey EKeys::Vive_Right_Trigger_Axis("Vive_Right_Trigger_Axis");
const FKey EKeys::Vive_Right_Trackpad_2D("Vive_Right_Trackpad_2D");
const FKey EKeys::Vive_Right_Trackpad_X("Vive_Right_Trackpad_X");
const FKey EKeys::Vive_Right_Trackpad_Y("Vive_Right_Trackpad_Y");
const FKey EKeys::Vive_Right_Trackpad_Click("Vive_Right_Trackpad_Click");
const FKey EKeys::Vive_Right_Trackpad_Touch("Vive_Right_Trackpad_Touch");
const FKey EKeys::Vive_Right_Trackpad_Up("Vive_Right_Trackpad_Up");
const FKey EKeys::Vive_Right_Trackpad_Down("Vive_Right_Trackpad_Down");
const FKey EKeys::Vive_Right_Trackpad_Left("Vive_Right_Trackpad_Left");
const FKey EKeys::Vive_Right_Trackpad_Right("Vive_Right_Trackpad_Right");

// Microsoft Mixed Reality Motion Controller
const FKey EKeys::MixedReality_Left_Menu_Click("MixedReality_Left_Menu_Click");
const FKey EKeys::MixedReality_Left_Grip_Click("MixedReality_Left_Grip_Click");
const FKey EKeys::MixedReality_Left_Trigger_Click("MixedReality_Left_Trigger_Click");
const FKey EKeys::MixedReality_Left_Trigger_Axis("MixedReality_Left_Trigger_Axis");
const FKey EKeys::MixedReality_Left_Thumbstick_2D("MixedReality_Left_Thumbstick_2D");
const FKey EKeys::MixedReality_Left_Thumbstick_X("MixedReality_Left_Thumbstick_X");
const FKey EKeys::MixedReality_Left_Thumbstick_Y("MixedReality_Left_Thumbstick_Y");
const FKey EKeys::MixedReality_Left_Thumbstick_Click("MixedReality_Left_Thumbstick_Click");
const FKey EKeys::MixedReality_Left_Thumbstick_Up("MixedReality_Left_Thumbstick_Up");
const FKey EKeys::MixedReality_Left_Thumbstick_Down("MixedReality_Left_Thumbstick_Down");
const FKey EKeys::MixedReality_Left_Thumbstick_Left("MixedReality_Left_Thumbstick_Left");
const FKey EKeys::MixedReality_Left_Thumbstick_Right("MixedReality_Left_Thumbstick_Right");
const FKey EKeys::MixedReality_Left_Trackpad_2D("MixedReality_Left_Trackpad_2D");
const FKey EKeys::MixedReality_Left_Trackpad_X("MixedReality_Left_Trackpad_X");
const FKey EKeys::MixedReality_Left_Trackpad_Y("MixedReality_Left_Trackpad_Y");
const FKey EKeys::MixedReality_Left_Trackpad_Click("MixedReality_Left_Trackpad_Click");
const FKey EKeys::MixedReality_Left_Trackpad_Touch("MixedReality_Left_Trackpad_Touch");
const FKey EKeys::MixedReality_Left_Trackpad_Up("MixedReality_Left_Trackpad_Up");
const FKey EKeys::MixedReality_Left_Trackpad_Down("MixedReality_Left_Trackpad_Down");
const FKey EKeys::MixedReality_Left_Trackpad_Left("MixedReality_Left_Trackpad_Left");
const FKey EKeys::MixedReality_Left_Trackpad_Right("MixedReality_Left_Trackpad_Right");
const FKey EKeys::MixedReality_Right_Menu_Click("MixedReality_Right_Menu_Click");
const FKey EKeys::MixedReality_Right_Grip_Click("MixedReality_Right_Grip_Click");
const FKey EKeys::MixedReality_Right_Trigger_Click("MixedReality_Right_Trigger_Click");
const FKey EKeys::MixedReality_Right_Trigger_Axis("MixedReality_Right_Trigger_Axis");
const FKey EKeys::MixedReality_Right_Thumbstick_2D("MixedReality_Right_Thumbstick_2D");
const FKey EKeys::MixedReality_Right_Thumbstick_X("MixedReality_Right_Thumbstick_X");
const FKey EKeys::MixedReality_Right_Thumbstick_Y("MixedReality_Right_Thumbstick_Y");
const FKey EKeys::MixedReality_Right_Thumbstick_Click("MixedReality_Right_Thumbstick_Click");
const FKey EKeys::MixedReality_Right_Thumbstick_Up("MixedReality_Right_Thumbstick_Up");
const FKey EKeys::MixedReality_Right_Thumbstick_Down("MixedReality_Right_Thumbstick_Down");
const FKey EKeys::MixedReality_Right_Thumbstick_Left("MixedReality_Right_Thumbstick_Left");
const FKey EKeys::MixedReality_Right_Thumbstick_Right("MixedReality_Right_Thumbstick_Right");
const FKey EKeys::MixedReality_Right_Trackpad_2D("MixedReality_Right_Trackpad_2D");
const FKey EKeys::MixedReality_Right_Trackpad_X("MixedReality_Right_Trackpad_X");
const FKey EKeys::MixedReality_Right_Trackpad_Y("MixedReality_Right_Trackpad_Y");
const FKey EKeys::MixedReality_Right_Trackpad_Click("MixedReality_Right_Trackpad_Click");
const FKey EKeys::MixedReality_Right_Trackpad_Touch("MixedReality_Right_Trackpad_Touch");
const FKey EKeys::MixedReality_Right_Trackpad_Up("MixedReality_Right_Trackpad_Up");
const FKey EKeys::MixedReality_Right_Trackpad_Down("MixedReality_Right_Trackpad_Down");
const FKey EKeys::MixedReality_Right_Trackpad_Left("MixedReality_Right_Trackpad_Left");
const FKey EKeys::MixedReality_Right_Trackpad_Right("MixedReality_Right_Trackpad_Right");

// Oculus Touch Controller
const FKey EKeys::OculusTouch_Left_X_Click("OculusTouch_Left_X_Click");
const FKey EKeys::OculusTouch_Left_Y_Click("OculusTouch_Left_Y_Click");
const FKey EKeys::OculusTouch_Left_X_Touch("OculusTouch_Left_X_Touch");
const FKey EKeys::OculusTouch_Left_Y_Touch("OculusTouch_Left_Y_Touch");
const FKey EKeys::OculusTouch_Left_Menu_Click("OculusTouch_Left_Menu_Click");
const FKey EKeys::OculusTouch_Left_Grip_Click("OculusTouch_Left_Grip_Click");
const FKey EKeys::OculusTouch_Left_Grip_Axis("OculusTouch_Left_Grip_Axis");
const FKey EKeys::OculusTouch_Left_Trigger_Click("OculusTouch_Left_Trigger_Click");
const FKey EKeys::OculusTouch_Left_Trigger_Axis("OculusTouch_Left_Trigger_Axis");
const FKey EKeys::OculusTouch_Left_Trigger_Touch("OculusTouch_Left_Trigger_Touch");
const FKey EKeys::OculusTouch_Left_Thumbstick_2D("OculusTouch_Left_Thumbstick_2D");
const FKey EKeys::OculusTouch_Left_Thumbstick_X("OculusTouch_Left_Thumbstick_X");
const FKey EKeys::OculusTouch_Left_Thumbstick_Y("OculusTouch_Left_Thumbstick_Y");
const FKey EKeys::OculusTouch_Left_Thumbstick_Click("OculusTouch_Left_Thumbstick_Click");
const FKey EKeys::OculusTouch_Left_Thumbstick_Touch("OculusTouch_Left_Thumbstick_Touch");
const FKey EKeys::OculusTouch_Left_Thumbstick_Up("OculusTouch_Left_Thumbstick_Up");
const FKey EKeys::OculusTouch_Left_Thumbstick_Down("OculusTouch_Left_Thumbstick_Down");
const FKey EKeys::OculusTouch_Left_Thumbstick_Left("OculusTouch_Left_Thumbstick_Left");
const FKey EKeys::OculusTouch_Left_Thumbstick_Right("OculusTouch_Left_Thumbstick_Right");
const FKey EKeys::OculusTouch_Right_A_Click("OculusTouch_Right_A_Click");
const FKey EKeys::OculusTouch_Right_B_Click("OculusTouch_Right_B_Click");
const FKey EKeys::OculusTouch_Right_A_Touch("OculusTouch_Right_A_Touch");
const FKey EKeys::OculusTouch_Right_B_Touch("OculusTouch_Right_B_Touch");
const FKey EKeys::OculusTouch_Right_Grip_Click("OculusTouch_Right_Grip_Click");
const FKey EKeys::OculusTouch_Right_Grip_Axis("OculusTouch_Right_Grip_Axis");
const FKey EKeys::OculusTouch_Right_Trigger_Click("OculusTouch_Right_Trigger_Click");
const FKey EKeys::OculusTouch_Right_Trigger_Axis("OculusTouch_Right_Trigger_Axis");
const FKey EKeys::OculusTouch_Right_Trigger_Touch("OculusTouch_Right_Trigger_Touch");
const FKey EKeys::OculusTouch_Right_Thumbstick_2D("OculusTouch_Right_Thumbstick_2D");
const FKey EKeys::OculusTouch_Right_Thumbstick_X("OculusTouch_Right_Thumbstick_X");
const FKey EKeys::OculusTouch_Right_Thumbstick_Y("OculusTouch_Right_Thumbstick_Y");
const FKey EKeys::OculusTouch_Right_Thumbstick_Click("OculusTouch_Right_Thumbstick_Click");
const FKey EKeys::OculusTouch_Right_Thumbstick_Touch("OculusTouch_Right_Thumbstick_Touch");
const FKey EKeys::OculusTouch_Right_Thumbstick_Up("OculusTouch_Right_Thumbstick_Up");
const FKey EKeys::OculusTouch_Right_Thumbstick_Down("OculusTouch_Right_Thumbstick_Down");
const FKey EKeys::OculusTouch_Right_Thumbstick_Left("OculusTouch_Right_Thumbstick_Left");
const FKey EKeys::OculusTouch_Right_Thumbstick_Right("OculusTouch_Right_Thumbstick_Right");

// Valve Index Controller
const FKey EKeys::ValveIndex_Left_A_Click("ValveIndex_Left_A_Click");
const FKey EKeys::ValveIndex_Left_B_Click("ValveIndex_Left_B_Click");
const FKey EKeys::ValveIndex_Left_A_Touch("ValveIndex_Left_A_Touch");
const FKey EKeys::ValveIndex_Left_B_Touch("ValveIndex_Left_B_Touch");
const FKey EKeys::ValveIndex_Left_Grip_Axis("ValveIndex_Left_Grip_Axis");
const FKey EKeys::ValveIndex_Left_Grip_Force("ValveIndex_Left_Grip_Force");
const FKey EKeys::ValveIndex_Left_Trigger_Click("ValveIndex_Left_Trigger_Click");
const FKey EKeys::ValveIndex_Left_Trigger_Axis("ValveIndex_Left_Trigger_Axis");
const FKey EKeys::ValveIndex_Left_Trigger_Touch("ValveIndex_Left_Trigger_Touch");
const FKey EKeys::ValveIndex_Left_Thumbstick_2D("ValveIndex_Left_Thumbstick_2D");
const FKey EKeys::ValveIndex_Left_Thumbstick_X("ValveIndex_Left_Thumbstick_X");
const FKey EKeys::ValveIndex_Left_Thumbstick_Y("ValveIndex_Left_Thumbstick_Y");
const FKey EKeys::ValveIndex_Left_Thumbstick_Click("ValveIndex_Left_Thumbstick_Click");
const FKey EKeys::ValveIndex_Left_Thumbstick_Touch("ValveIndex_Left_Thumbstick_Touch");
const FKey EKeys::ValveIndex_Left_Thumbstick_Up("ValveIndex_Left_Thumbstick_Up");
const FKey EKeys::ValveIndex_Left_Thumbstick_Down("ValveIndex_Left_Thumbstick_Down");
const FKey EKeys::ValveIndex_Left_Thumbstick_Left("ValveIndex_Left_Thumbstick_Left");
const FKey EKeys::ValveIndex_Left_Thumbstick_Right("ValveIndex_Left_Thumbstick_Right");
const FKey EKeys::ValveIndex_Left_Trackpad_2D("ValveIndex_Left_Trackpad_2D");
const FKey EKeys::ValveIndex_Left_Trackpad_X("ValveIndex_Left_Trackpad_X");
const FKey EKeys::ValveIndex_Left_Trackpad_Y("ValveIndex_Left_Trackpad_Y");
const FKey EKeys::ValveIndex_Left_Trackpad_Force("ValveIndex_Left_Trackpad_Force");
const FKey EKeys::ValveIndex_Left_Trackpad_Touch("ValveIndex_Left_Trackpad_Touch");
const FKey EKeys::ValveIndex_Left_Trackpad_Up("ValveIndex_Left_Trackpad_Up");
const FKey EKeys::ValveIndex_Left_Trackpad_Down("ValveIndex_Left_Trackpad_Down");
const FKey EKeys::ValveIndex_Left_Trackpad_Left("ValveIndex_Left_Trackpad_Left");
const FKey EKeys::ValveIndex_Left_Trackpad_Right("ValveIndex_Left_Trackpad_Right");
const FKey EKeys::ValveIndex_Right_A_Click("ValveIndex_Right_A_Click");
const FKey EKeys::ValveIndex_Right_B_Click("ValveIndex_Right_B_Click");
const FKey EKeys::ValveIndex_Right_A_Touch("ValveIndex_Right_A_Touch");
const FKey EKeys::ValveIndex_Right_B_Touch("ValveIndex_Right_B_Touch");
const FKey EKeys::ValveIndex_Right_Grip_Axis("ValveIndex_Right_Grip_Axis");
const FKey EKeys::ValveIndex_Right_Grip_Force("ValveIndex_Right_Grip_Force");
const FKey EKeys::ValveIndex_Right_Trigger_Click("ValveIndex_Right_Trigger_Click");
const FKey EKeys::ValveIndex_Right_Trigger_Axis("ValveIndex_Right_Trigger_Axis");
const FKey EKeys::ValveIndex_Right_Trigger_Touch("ValveIndex_Right_Trigger_Touch");
const FKey EKeys::ValveIndex_Right_Thumbstick_2D("ValveIndex_Right_Thumbstick_2D");
const FKey EKeys::ValveIndex_Right_Thumbstick_X("ValveIndex_Right_Thumbstick_X");
const FKey EKeys::ValveIndex_Right_Thumbstick_Y("ValveIndex_Right_Thumbstick_Y");
const FKey EKeys::ValveIndex_Right_Thumbstick_Click("ValveIndex_Right_Thumbstick_Click");
const FKey EKeys::ValveIndex_Right_Thumbstick_Touch("ValveIndex_Right_Thumbstick_Touch");
const FKey EKeys::ValveIndex_Right_Thumbstick_Up("ValveIndex_Right_Thumbstick_Up");
const FKey EKeys::ValveIndex_Right_Thumbstick_Down("ValveIndex_Right_Thumbstick_Down");
const FKey EKeys::ValveIndex_Right_Thumbstick_Left("ValveIndex_Right_Thumbstick_Left");
const FKey EKeys::ValveIndex_Right_Thumbstick_Right("ValveIndex_Right_Thumbstick_Right");
const FKey EKeys::ValveIndex_Right_Trackpad_2D("ValveIndex_Right_Trackpad_2D");
const FKey EKeys::ValveIndex_Right_Trackpad_X("ValveIndex_Right_Trackpad_X");
const FKey EKeys::ValveIndex_Right_Trackpad_Y("ValveIndex_Right_Trackpad_Y");
const FKey EKeys::ValveIndex_Right_Trackpad_Force("ValveIndex_Right_Trackpad_Force");
const FKey EKeys::ValveIndex_Right_Trackpad_Touch("ValveIndex_Right_Trackpad_Touch");
const FKey EKeys::ValveIndex_Right_Trackpad_Up("ValveIndex_Right_Trackpad_Up");
const FKey EKeys::ValveIndex_Right_Trackpad_Down("ValveIndex_Right_Trackpad_Down");
const FKey EKeys::ValveIndex_Right_Trackpad_Left("ValveIndex_Right_Trackpad_Left");
const FKey EKeys::ValveIndex_Right_Trackpad_Right("ValveIndex_Right_Trackpad_Right");

const FKey EKeys::Invalid(NAME_None);

const FName EKeys::NAME_GamepadCategory("Gamepad");
const FName EKeys::NAME_MouseCategory("Mouse");
const FName EKeys::NAME_KeyboardCategory("Key");
const FName EKeys::NAME_VirtualCategory("Virtual");

// Create the virtual FKeys from the platform specified versions of their keys
PRAGMA_DISABLE_DEPRECATION_WARNINGS
const FKey EKeys::Virtual_Accept = FPlatformInput::GetGamepadAcceptKey();
const FKey EKeys::Virtual_Back = FPlatformInput::GetGamepadBackKey();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

const FKey EKeys::Virtual_Gamepad_Accept("Virtual_Gamepad_Accept");
const FKey EKeys::Virtual_Gamepad_Back("Virtual_Gamepad_Back");

bool EKeys::bInitialized = false;
TMap<FKey, TSharedPtr<FKeyDetails>> EKeys::InputKeys;
TMap<FKey, EKeys::FPairedKeyDetails> EKeys::PairedInputKeysLookup;
TMap<FName, EKeys::FCategoryDisplayInfo> EKeys::MenuCategoryDisplayInfo;

FKeyDetails::FKeyDetails(const FKey InKey, const TAttribute<FText>& InLongDisplayName, const TAttribute<FText>& InShortDisplayName, const uint32 InKeyFlags, const FName InMenuCategory)
	: Key(InKey)
	, MenuCategory(InMenuCategory)
	, LongDisplayName(InLongDisplayName)
	, ShortDisplayName(InShortDisplayName)
{
	CommonInit(InKeyFlags);
}

FKeyDetails::FKeyDetails(const FKey InKey, const TAttribute<FText>& InLongDisplayName, const uint32 InKeyFlags, const FName InMenuCategory, const TAttribute<FText>& InShortDisplayName)
	: Key(InKey)
	, MenuCategory(InMenuCategory)
	, LongDisplayName(InLongDisplayName)
	, ShortDisplayName(InShortDisplayName)
{
	CommonInit(InKeyFlags);
}

void FKeyDetails::CommonInit(const uint32 InKeyFlags)
{
	bIsModifierKey = ((InKeyFlags & EKeyFlags::ModifierKey) != 0);
	bIsGamepadKey = ((InKeyFlags & EKeyFlags::GamepadKey) != 0);
	bIsTouch = ((InKeyFlags & EKeyFlags::Touch) != 0);
	bIsMouseButton = ((InKeyFlags & EKeyFlags::MouseButton) != 0);
	bIsBindableInBlueprints = ((~InKeyFlags & EKeyFlags::NotBlueprintBindableKey) != 0) && ((~InKeyFlags & EKeyFlags::Deprecated) != 0);
	bShouldUpdateAxisWithoutSamples = ((InKeyFlags & EKeyFlags::UpdateAxisWithoutSamples) != 0);
	bIsBindableToActions = ((~InKeyFlags & EKeyFlags::NotActionBindableKey) != 0) && ((~InKeyFlags & EKeyFlags::Deprecated) != 0);
	bIsDeprecated = ((InKeyFlags & EKeyFlags::Deprecated) != 0);
	bIsGesture = ((InKeyFlags & EKeyFlags::Gesture) != 0);
	bIsVirtual = ((InKeyFlags & EKeyFlags::Virtual) != 0);

	if ((InKeyFlags & EKeyFlags::ButtonAxis) != 0)
	{
		ensure((InKeyFlags & (EKeyFlags::Axis1D | EKeyFlags::Axis2D | EKeyFlags::Axis3D)) == 0);
		AxisType = EInputAxisType::Button;
	}
	else if ((InKeyFlags & EKeyFlags::Axis1D) != 0)
	{
		ensure((InKeyFlags & (EKeyFlags::Axis2D | EKeyFlags::Axis3D)) == 0);
		AxisType = EInputAxisType::Axis1D;
	}
	else if ((InKeyFlags & EKeyFlags::Axis2D) != 0)
	{
		ensure((InKeyFlags & EKeyFlags::Axis3D) == 0);
		AxisType = EInputAxisType::Axis2D;
	}
	else if ((InKeyFlags & EKeyFlags::Axis3D) != 0)
	{
		AxisType = EInputAxisType::Axis3D;
	}
	else
	{
		AxisType = EInputAxisType::None;
	}

	// Set up default menu categories
	if (MenuCategory.IsNone())
	{
		if (IsVirtual())
		{
			MenuCategory = EKeys::NAME_VirtualCategory;
		}
		else if (IsGamepadKey())
		{
			MenuCategory = EKeys::NAME_GamepadCategory;
		}
		else if (IsMouseButton())
		{
			MenuCategory = EKeys::NAME_MouseCategory;
		}
		else
		{
			MenuCategory = EKeys::NAME_KeyboardCategory;
		}
	}
}

#if !FAST_BOOT_HACKS
UE_DISABLE_OPTIMIZATION_SHIP
#endif
void EKeys::Initialize()
{
	if (bInitialized) return;

	LLM_SCOPE_BYTAG(InputCore_EKeys);
	
	bInitialized = true;

	for (int32 TouchIndex = 0; TouchIndex < (EKeys::NUM_TOUCH_KEYS - 1); TouchIndex++)
	{
		const_cast<FKey&>(EKeys::TouchKeys[TouchIndex]) = FKey(*FString::Printf(TEXT("Touch%d"), TouchIndex + 1));
	}

	AddMenuCategoryDisplayInfo(NAME_GamepadCategory, LOCTEXT("GamepadSubCategory", "Gamepad"), TEXT("GraphEditor.PadEvent_16x"));
	AddMenuCategoryDisplayInfo(NAME_VirtualCategory, LOCTEXT("VirtualSubCategory", "Virtual"), TEXT("GraphEditor.PadEvent_16x"));
	AddMenuCategoryDisplayInfo(NAME_MouseCategory, LOCTEXT("MouseSubCategory", "Mouse"), TEXT("GraphEditor.MouseEvent_16x"));
	AddMenuCategoryDisplayInfo(NAME_KeyboardCategory, LOCTEXT("KeyboardSubCategory", "Keyboard"), TEXT("GraphEditor.KeyEvent_16x"));

	AddKey(FKeyDetails(EKeys::AnyKey, LOCTEXT("AnyKey", "Any Key")));

	AddKey(FKeyDetails(EKeys::MouseX, LOCTEXT("MouseX", "Mouse X"), FKeyDetails::Axis1D | FKeyDetails::MouseButton | FKeyDetails::UpdateAxisWithoutSamples));
	AddKey(FKeyDetails(EKeys::MouseY, LOCTEXT("MouseY", "Mouse Y"), FKeyDetails::Axis1D | FKeyDetails::MouseButton | FKeyDetails::UpdateAxisWithoutSamples));
	AddPairedKey(FKeyDetails(EKeys::Mouse2D, LOCTEXT("Mouse2D", "Mouse XY 2D-Axis"), FKeyDetails::Axis2D | FKeyDetails::MouseButton | FKeyDetails::UpdateAxisWithoutSamples), EKeys::MouseX, EKeys::MouseY);
	AddKey(FKeyDetails(EKeys::MouseWheelAxis, LOCTEXT("MouseWheelAxis", "Mouse Wheel Axis"), FKeyDetails::Axis1D | FKeyDetails::MouseButton | FKeyDetails::UpdateAxisWithoutSamples));
	AddKey(FKeyDetails(EKeys::MouseScrollUp, LOCTEXT("MouseScrollUp", "Mouse Wheel Up"), FKeyDetails::MouseButton | FKeyDetails::ButtonAxis));
	AddKey(FKeyDetails(EKeys::MouseScrollDown, LOCTEXT("MouseScrollDown", "Mouse Wheel Down"), FKeyDetails::MouseButton | FKeyDetails::ButtonAxis));

	AddKey(FKeyDetails(EKeys::LeftMouseButton, LOCTEXT("LeftMouseButton", "Left Mouse Button"), FKeyDetails::MouseButton, NAME_None, LOCTEXT("LeftMouseButtonShort", "LMB")));
	AddKey(FKeyDetails(EKeys::RightMouseButton, LOCTEXT("RightMouseButton", "Right Mouse Button"), FKeyDetails::MouseButton, NAME_None, LOCTEXT("RightMouseButtonShort", "RMB")));
	AddKey(FKeyDetails(EKeys::MiddleMouseButton, LOCTEXT("MiddleMouseButton", "Middle Mouse Button"), FKeyDetails::MouseButton));
	AddKey(FKeyDetails(EKeys::ThumbMouseButton, LOCTEXT("ThumbMouseButton", "Thumb Mouse Button"), FKeyDetails::MouseButton));
	AddKey(FKeyDetails(EKeys::ThumbMouseButton2, LOCTEXT("ThumbMouseButton2", "Thumb Mouse Button 2"), FKeyDetails::MouseButton));

	AddKey(FKeyDetails(EKeys::Tab, LOCTEXT("Tab", "Tab")));
	AddKey(FKeyDetails(EKeys::Enter, LOCTEXT("Enter", "Enter")));
	AddKey(FKeyDetails(EKeys::Pause, LOCTEXT("Pause", "Pause")));

	AddKey(FKeyDetails(EKeys::CapsLock, LOCTEXT("CapsLock", "Caps Lock"), LOCTEXT("CapsLockShort", "Caps")));
	AddKey(FKeyDetails(EKeys::Escape, LOCTEXT("Escape", "Escape"), LOCTEXT("EscapeShort", "Esc")));
	AddKey(FKeyDetails(EKeys::SpaceBar, LOCTEXT("SpaceBar", "Space Bar"), LOCTEXT("SpaceBarShort", "Space")));
	AddKey(FKeyDetails(EKeys::PageUp, LOCTEXT("PageUp", "Page Up"), LOCTEXT("PageUpShort", "PgUp")));
	AddKey(FKeyDetails(EKeys::PageDown, LOCTEXT("PageDown", "Page Down"), LOCTEXT("PageDownShort", "PgDn")));
	AddKey(FKeyDetails(EKeys::End, LOCTEXT("End", "End")));
	AddKey(FKeyDetails(EKeys::Home, LOCTEXT("Home", "Home")));

	AddKey(FKeyDetails(EKeys::Left, LOCTEXT("Left", "Left")));
	AddKey(FKeyDetails(EKeys::Up, LOCTEXT("Up", "Up")));
	AddKey(FKeyDetails(EKeys::Right, LOCTEXT("Right", "Right")));
	AddKey(FKeyDetails(EKeys::Down, LOCTEXT("Down", "Down")));

	AddKey(FKeyDetails(EKeys::Insert, LOCTEXT("Insert", "Insert"), LOCTEXT("InsertShort", "Ins")));

#if PLATFORM_MAC
    AddKey(FKeyDetails(EKeys::BackSpace, LOCTEXT("Delete", "Delete"), LOCTEXT("DeleteShort", "Del")));
    AddKey(FKeyDetails(EKeys::Delete, LOCTEXT("ForwardDelete", "Fn+Delete")));
#else
    AddKey(FKeyDetails(EKeys::BackSpace, LOCTEXT("BackSpace", "Backspace")));
    AddKey(FKeyDetails(EKeys::Delete, LOCTEXT("Delete", "Delete"), LOCTEXT("DeleteShort", "Del")));
#endif

	AddKey(FKeyDetails(EKeys::Zero, FText::FromString("0")));
	AddKey(FKeyDetails(EKeys::One, FText::FromString("1")));
	AddKey(FKeyDetails(EKeys::Two, FText::FromString("2")));
	AddKey(FKeyDetails(EKeys::Three, FText::FromString("3")));
	AddKey(FKeyDetails(EKeys::Four, FText::FromString("4")));
	AddKey(FKeyDetails(EKeys::Five, FText::FromString("5")));
	AddKey(FKeyDetails(EKeys::Six, FText::FromString("6")));
	AddKey(FKeyDetails(EKeys::Seven, FText::FromString("7")));
	AddKey(FKeyDetails(EKeys::Eight, FText::FromString("8")));
	AddKey(FKeyDetails(EKeys::Nine, FText::FromString("9")));

	AddKey(FKeyDetails(EKeys::A, FText::FromString("A")));
	AddKey(FKeyDetails(EKeys::B, FText::FromString("B")));
	AddKey(FKeyDetails(EKeys::C, FText::FromString("C")));
	AddKey(FKeyDetails(EKeys::D, FText::FromString("D")));
	AddKey(FKeyDetails(EKeys::E, FText::FromString("E")));
	AddKey(FKeyDetails(EKeys::F, FText::FromString("F")));
	AddKey(FKeyDetails(EKeys::G, FText::FromString("G")));
	AddKey(FKeyDetails(EKeys::H, FText::FromString("H")));
	AddKey(FKeyDetails(EKeys::I, FText::FromString("I")));
	AddKey(FKeyDetails(EKeys::J, FText::FromString("J")));
	AddKey(FKeyDetails(EKeys::K, FText::FromString("K")));
	AddKey(FKeyDetails(EKeys::L, FText::FromString("L")));
	AddKey(FKeyDetails(EKeys::M, FText::FromString("M")));
	AddKey(FKeyDetails(EKeys::N, FText::FromString("N")));
	AddKey(FKeyDetails(EKeys::O, FText::FromString("O")));
	AddKey(FKeyDetails(EKeys::P, FText::FromString("P")));
	AddKey(FKeyDetails(EKeys::Q, FText::FromString("Q")));
	AddKey(FKeyDetails(EKeys::R, FText::FromString("R")));
	AddKey(FKeyDetails(EKeys::S, FText::FromString("S")));
	AddKey(FKeyDetails(EKeys::T, FText::FromString("T")));
	AddKey(FKeyDetails(EKeys::U, FText::FromString("U")));
	AddKey(FKeyDetails(EKeys::V, FText::FromString("V")));
	AddKey(FKeyDetails(EKeys::W, FText::FromString("W")));
	AddKey(FKeyDetails(EKeys::X, FText::FromString("X")));
	AddKey(FKeyDetails(EKeys::Y, FText::FromString("Y")));
	AddKey(FKeyDetails(EKeys::Z, FText::FromString("Z")));

	AddKey(FKeyDetails(EKeys::NumPadZero, LOCTEXT("NumPadZero", "Num 0")));
	AddKey(FKeyDetails(EKeys::NumPadOne, LOCTEXT("NumPadOne", "Num 1")));
	AddKey(FKeyDetails(EKeys::NumPadTwo, LOCTEXT("NumPadTwo", "Num 2")));
	AddKey(FKeyDetails(EKeys::NumPadThree, LOCTEXT("NumPadThree", "Num 3")));
	AddKey(FKeyDetails(EKeys::NumPadFour, LOCTEXT("NumPadFour", "Num 4")));
	AddKey(FKeyDetails(EKeys::NumPadFive, LOCTEXT("NumPadFive", "Num 5")));
	AddKey(FKeyDetails(EKeys::NumPadSix, LOCTEXT("NumPadSix", "Num 6")));
	AddKey(FKeyDetails(EKeys::NumPadSeven, LOCTEXT("NumPadSeven", "Num 7")));
	AddKey(FKeyDetails(EKeys::NumPadEight, LOCTEXT("NumPadEight", "Num 8")));
	AddKey(FKeyDetails(EKeys::NumPadNine, LOCTEXT("NumPadNine", "Num 9")));

	AddKey(FKeyDetails(EKeys::Multiply, LOCTEXT("Multiply", "Num *")));
	AddKey(FKeyDetails(EKeys::Add, LOCTEXT("Add", "Num +")));
	AddKey(FKeyDetails(EKeys::Subtract, LOCTEXT("Subtract", "Num -")));
	AddKey(FKeyDetails(EKeys::Decimal, LOCTEXT("Decimal", "Num .")));
	AddKey(FKeyDetails(EKeys::Divide, LOCTEXT("Divide", "Num /")));

	AddKey(FKeyDetails(EKeys::F1, LOCTEXT("F1", "F1")));
	AddKey(FKeyDetails(EKeys::F2, LOCTEXT("F2", "F2")));
	AddKey(FKeyDetails(EKeys::F3, LOCTEXT("F3", "F3")));
	AddKey(FKeyDetails(EKeys::F4, LOCTEXT("F4", "F4")));
	AddKey(FKeyDetails(EKeys::F5, LOCTEXT("F5", "F5")));
	AddKey(FKeyDetails(EKeys::F6, LOCTEXT("F6", "F6")));
	AddKey(FKeyDetails(EKeys::F7, LOCTEXT("F7", "F7")));
	AddKey(FKeyDetails(EKeys::F8, LOCTEXT("F8", "F8")));
	AddKey(FKeyDetails(EKeys::F9, LOCTEXT("F9", "F9")));
	AddKey(FKeyDetails(EKeys::F10, LOCTEXT("F10", "F10")));
	AddKey(FKeyDetails(EKeys::F11, LOCTEXT("F11", "F11")));
	AddKey(FKeyDetails(EKeys::F12, LOCTEXT("F12", "F12")));

	AddKey(FKeyDetails(EKeys::NumLock, LOCTEXT("NumLock", "Num Lock")));
	AddKey(FKeyDetails(EKeys::ScrollLock, LOCTEXT("ScrollLock", "Scroll Lock")));

	AddKey(FKeyDetails(EKeys::LeftShift, LOCTEXT("LeftShift", "Left Shift"), FKeyDetails::ModifierKey));
	AddKey(FKeyDetails(EKeys::RightShift, LOCTEXT("RightShift", "Right Shift"), FKeyDetails::ModifierKey));
	AddKey(FKeyDetails(EKeys::LeftControl, LOCTEXT("LeftControl", "Left Ctrl"), FKeyDetails::ModifierKey));
	AddKey(FKeyDetails(EKeys::RightControl, LOCTEXT("RightControl", "Right Ctrl"), FKeyDetails::ModifierKey));
	AddKey(FKeyDetails(EKeys::LeftAlt, LOCTEXT("LeftAlt", "Left Alt"), FKeyDetails::ModifierKey));
	AddKey(FKeyDetails(EKeys::RightAlt, LOCTEXT("RightAlt", "Right Alt"), FKeyDetails::ModifierKey));
	AddKey(FKeyDetails(EKeys::LeftCommand, LOCTEXT("LeftCommand", "Left Cmd"), FKeyDetails::ModifierKey));
	AddKey(FKeyDetails(EKeys::RightCommand, LOCTEXT("RightCommand", "Right Cmd"), FKeyDetails::ModifierKey));

	AddKey(FKeyDetails(EKeys::Semicolon, LOCTEXT("Semicolon", "Semicolon"), FText::FromString(";")));
	AddKey(FKeyDetails(EKeys::Equals, LOCTEXT("Equals", "Equals"), FText::FromString("=")));
	AddKey(FKeyDetails(EKeys::Comma, LOCTEXT("Comma", "Comma"), FText::FromString(",")));
	AddKey(FKeyDetails(EKeys::Hyphen, LOCTEXT("Hyphen", "Hyphen"), FText::FromString("-")));
	AddKey(FKeyDetails(EKeys::Underscore, LOCTEXT("Underscore", "Underscore"), FText::FromString("_")));
	AddKey(FKeyDetails(EKeys::Period, LOCTEXT("Period", "Period"), FText::FromString(".")));
	AddKey(FKeyDetails(EKeys::Slash, LOCTEXT("Slash", "Slash"), FText::FromString("/")));
	AddKey(FKeyDetails(EKeys::Tilde, FText::FromString("`"))); // Yes this is not actually a tilde, it is a long, sad, and old story
	AddKey(FKeyDetails(EKeys::LeftBracket, LOCTEXT("LeftBracket", "Left Bracket"), FText::FromString("[")));
	AddKey(FKeyDetails(EKeys::Backslash, LOCTEXT("Backslash", "Backslash"), FText::FromString("\\")));
	AddKey(FKeyDetails(EKeys::RightBracket, LOCTEXT("RightBracket", "Right Bracket"), FText::FromString("]")));
	AddKey(FKeyDetails(EKeys::Apostrophe, LOCTEXT("Apostrophe", "Apostrophe"), FText::FromString("'")));
	AddKey(FKeyDetails(EKeys::Quote, LOCTEXT("Quote", "Quote"), FText::FromString("\"")));

	AddKey(FKeyDetails(EKeys::LeftParantheses, LOCTEXT("LeftParantheses", "Left Parantheses"), FText::FromString("(")));
	AddKey(FKeyDetails(EKeys::RightParantheses, LOCTEXT("RightParantheses", "Right Parantheses"), FText::FromString(")")));
	AddKey(FKeyDetails(EKeys::Ampersand, LOCTEXT("Ampersand", "Ampersand"), FText::FromString("&")));
	AddKey(FKeyDetails(EKeys::Asterix, LOCTEXT("Asterix", "Asterisk"), FText::FromString("*")));
	AddKey(FKeyDetails(EKeys::Caret, LOCTEXT("Caret", "Caret"), FText::FromString("^")));
	AddKey(FKeyDetails(EKeys::Dollar, LOCTEXT("Dollar", "Dollar"), FText::FromString("$")));
	AddKey(FKeyDetails(EKeys::Exclamation, LOCTEXT("Exclamation", "Exclamation"), FText::FromString("!")));
	AddKey(FKeyDetails(EKeys::Colon, LOCTEXT("Colon", "Colon"), FText::FromString(":")));

	AddKey(FKeyDetails(EKeys::A_AccentGrave, FText::FromString(FString::Chr(224))));
	AddKey(FKeyDetails(EKeys::E_AccentGrave, FText::FromString(FString::Chr(232))));
	AddKey(FKeyDetails(EKeys::E_AccentAigu, FText::FromString(FString::Chr(233))));
	AddKey(FKeyDetails(EKeys::C_Cedille, FText::FromString(FString::Chr(231))));
	AddKey(FKeyDetails(EKeys::Section, FText::FromString(FString::Chr(167))));


	// Setup Gamepad keys
	AddKey(FKeyDetails(EKeys::Gamepad_LeftX, LOCTEXT("Gamepad_LeftX", "Gamepad Left Thumbstick X-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D));
	AddKey(FKeyDetails(EKeys::Gamepad_LeftY, LOCTEXT("Gamepad_LeftY", "Gamepad Left Thumbstick Y-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D));
	AddPairedKey(FKeyDetails(EKeys::Gamepad_Left2D, LOCTEXT("Gamepad_Left2D", "Gamepad Left Thumbstick 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D), EKeys::Gamepad_LeftX, EKeys::Gamepad_LeftY);
	AddKey(FKeyDetails(EKeys::Gamepad_RightX, LOCTEXT("Gamepad_RightX", "Gamepad Right Thumbstick X-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D));
	AddKey(FKeyDetails(EKeys::Gamepad_RightY, LOCTEXT("Gamepad_RightY", "Gamepad Right Thumbstick Y-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D));
	AddPairedKey(FKeyDetails(EKeys::Gamepad_Right2D, LOCTEXT("Gamepad_Right2D", "Gamepad Right Thumbstick 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D), EKeys::Gamepad_RightX, EKeys::Gamepad_RightY);

	AddKey(FKeyDetails(EKeys::Gamepad_DPad_Up, LOCTEXT("Gamepad_DPad_Up", "Gamepad D-pad Up"), FKeyDetails::GamepadKey));
	AddKey(FKeyDetails(EKeys::Gamepad_DPad_Down, LOCTEXT("Gamepad_DPad_Down", "Gamepad D-pad Down"), FKeyDetails::GamepadKey));
	AddKey(FKeyDetails(EKeys::Gamepad_DPad_Right, LOCTEXT("Gamepad_DPad_Right", "Gamepad D-pad Right"), FKeyDetails::GamepadKey));
	AddKey(FKeyDetails(EKeys::Gamepad_DPad_Left, LOCTEXT("Gamepad_DPad_Left", "Gamepad D-pad Left"), FKeyDetails::GamepadKey));

	// Virtual key codes used for input axis button press/release emulation
	AddKey(FKeyDetails(EKeys::Gamepad_LeftStick_Up, LOCTEXT("Gamepad_LeftStick_Up", "Gamepad Left Thumbstick Up"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis));
	AddKey(FKeyDetails(EKeys::Gamepad_LeftStick_Down, LOCTEXT("Gamepad_LeftStick_Down", "Gamepad Left Thumbstick Down"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis));
	AddKey(FKeyDetails(EKeys::Gamepad_LeftStick_Right, LOCTEXT("Gamepad_LeftStick_Right", "Gamepad Left Thumbstick Right"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis));
	AddKey(FKeyDetails(EKeys::Gamepad_LeftStick_Left, LOCTEXT("Gamepad_LeftStick_Left", "Gamepad Left Thumbstick Left"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis));

	AddKey(FKeyDetails(EKeys::Gamepad_RightStick_Up, LOCTEXT("Gamepad_RightStick_Up", "Gamepad Right Thumbstick Up"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis));
	AddKey(FKeyDetails(EKeys::Gamepad_RightStick_Down, LOCTEXT("Gamepad_RightStick_Down", "Gamepad Right Thumbstick Down"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis));
	AddKey(FKeyDetails(EKeys::Gamepad_RightStick_Right, LOCTEXT("Gamepad_RightStick_Right", "Gamepad Right Thumbstick Right"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis));
	AddKey(FKeyDetails(EKeys::Gamepad_RightStick_Left, LOCTEXT("Gamepad_RightStick_Left", "Gamepad Right Thumbstick Left"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis));

	AddKey(FKeyDetails(EKeys::Gamepad_Special_Left, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&EKeys::GetGamepadDisplayName, EKeys::Gamepad_Special_Left)), FKeyDetails::GamepadKey));
	AddKey(FKeyDetails(EKeys::Gamepad_Special_Right, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&EKeys::GetGamepadDisplayName, EKeys::Gamepad_Special_Right)), FKeyDetails::GamepadKey));
	AddKey(FKeyDetails(EKeys::Gamepad_FaceButton_Bottom, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&EKeys::GetGamepadDisplayName, EKeys::Gamepad_FaceButton_Bottom)), FKeyDetails::GamepadKey));
	AddKey(FKeyDetails(EKeys::Gamepad_FaceButton_Right, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&EKeys::GetGamepadDisplayName, EKeys::Gamepad_FaceButton_Right)), FKeyDetails::GamepadKey));
	AddKey(FKeyDetails(EKeys::Gamepad_FaceButton_Left, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&EKeys::GetGamepadDisplayName, EKeys::Gamepad_FaceButton_Left)), FKeyDetails::GamepadKey));
	AddKey(FKeyDetails(EKeys::Gamepad_FaceButton_Top, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&EKeys::GetGamepadDisplayName, EKeys::Gamepad_FaceButton_Top)), FKeyDetails::GamepadKey));

	AddKey(FKeyDetails(EKeys::Gamepad_LeftTriggerAxis, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&EKeys::GetGamepadDisplayName, EKeys::Gamepad_LeftTriggerAxis)), FKeyDetails::GamepadKey | FKeyDetails::Axis1D));
	AddKey(FKeyDetails(EKeys::Gamepad_RightTriggerAxis, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&EKeys::GetGamepadDisplayName, EKeys::Gamepad_RightTriggerAxis)), FKeyDetails::GamepadKey | FKeyDetails::Axis1D));

	AddKey(FKeyDetails(EKeys::Gamepad_LeftShoulder, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&EKeys::GetGamepadDisplayName, EKeys::Gamepad_LeftShoulder)), FKeyDetails::GamepadKey));
	AddKey(FKeyDetails(EKeys::Gamepad_RightShoulder, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&EKeys::GetGamepadDisplayName, EKeys::Gamepad_RightShoulder)), FKeyDetails::GamepadKey));
	AddKey(FKeyDetails(EKeys::Gamepad_LeftTrigger, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&EKeys::GetGamepadDisplayName, EKeys::Gamepad_LeftTrigger)), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis));
	AddKey(FKeyDetails(EKeys::Gamepad_RightTrigger, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&EKeys::GetGamepadDisplayName, EKeys::Gamepad_RightTrigger)), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis));

	AddKey(FKeyDetails(EKeys::Gamepad_LeftThumbstick, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&EKeys::GetGamepadDisplayName, EKeys::Gamepad_LeftThumbstick)), FKeyDetails::GamepadKey));
	AddKey(FKeyDetails(EKeys::Gamepad_RightThumbstick, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&EKeys::GetGamepadDisplayName, EKeys::Gamepad_RightThumbstick)), FKeyDetails::GamepadKey));

	// The virtual accept and back keys represent platform specific actions which may have different bindings per-platform.
	AddVirtualKey(FKeyDetails(EKeys::Virtual_Gamepad_Accept, LOCTEXT("Virtual_Gamepad_Accept_Key", "Virtual Gamepad Accept"), FKeyDetails::GamepadKey | FKeyDetails::Virtual, EKeys::NAME_VirtualCategory), FPlatformInput::GetGamepadAcceptKey());
	AddVirtualKey(FKeyDetails(EKeys::Virtual_Gamepad_Back, LOCTEXT("Virtual_Gamepad_Back_Key", "Virtual Gamepad Back"), FKeyDetails::GamepadKey | FKeyDetails::Virtual, EKeys::NAME_VirtualCategory), FPlatformInput::GetGamepadBackKey());
	
	AddMenuCategoryDisplayInfo("Motion", LOCTEXT("MotionSubCateogry", "Motion"), TEXT("GraphEditor.KeyEvent_16x"));

	AddKey(FKeyDetails(EKeys::Tilt, LOCTEXT("Tilt", "Tilt"), FKeyDetails::Axis3D, "Motion"));
	AddKey(FKeyDetails(EKeys::RotationRate, LOCTEXT("RotationRate", "Rotation Rate"), FKeyDetails::Axis3D, "Motion"));
	AddKey(FKeyDetails(EKeys::Gravity, LOCTEXT("Gravity", "Gravity"), FKeyDetails::Axis3D, "Motion"));
	AddKey(FKeyDetails(EKeys::Acceleration, LOCTEXT("Acceleration", "Acceleration"), FKeyDetails::Axis3D, "Motion"));

	// Fingers
	AddMenuCategoryDisplayInfo("Touch", LOCTEXT("TouchSubCateogry", "Touch"), TEXT("GraphEditor.TouchEvent_16x"));

	static_assert(EKeys::NUM_TOUCH_KEYS == ETouchIndex::MAX_TOUCHES, "The number of touch keys should be equal to the max number of TouchIndexes");
	for (int TouchIndex = 0; TouchIndex < (EKeys::NUM_TOUCH_KEYS - 1); TouchIndex++)
	{
		AddKey(FKeyDetails(EKeys::TouchKeys[TouchIndex], FText::Format(LOCTEXT("TouchFormat", "Touch {0}"), FText::AsNumber(TouchIndex + 1)), FKeyDetails::Touch, "Touch"));
	}

	// Make sure the Touch key for the cursor pointer is invalid, we don't want there to be an FKey for "Touch (Mouse Index)".
	check(!EKeys::TouchKeys[(int32)ETouchIndex::CursorPointerIndex].IsValid());

	// Gestures
	AddMenuCategoryDisplayInfo("Gesture", LOCTEXT("GestureSubCateogry", "Gesture"), TEXT("GraphEditor.KeyEvent_16x"));

	AddKey(FKeyDetails(EKeys::Gesture_Pinch, LOCTEXT("Gesture_Pinch", "Pinch"), FKeyDetails::Gesture | FKeyDetails::Axis1D, "Gesture"));
	AddKey(FKeyDetails(EKeys::Gesture_Flick, LOCTEXT("Gesture_Flick", "Flick"), FKeyDetails::Gesture | FKeyDetails::Axis1D, "Gesture"));
	AddKey(FKeyDetails(EKeys::Gesture_Rotate, LOCTEXT("Gesture_Rotate", "Rotate"), FKeyDetails::Gesture | FKeyDetails::Axis1D, "Gesture"));
	AddKey(FKeyDetails(EKeys::Gesture_Pan, LOCTEXT("Gesture_Pan", "Pan"), FKeyDetails::Gesture | FKeyDetails::Axis2D, "Gesture"));
	AddKey(FKeyDetails(EKeys::Gesture_Tap, LOCTEXT("Gesture_Tap", "Tap"), FKeyDetails::Gesture | FKeyDetails::Axis2D, "Gesture"));
	AddKey(FKeyDetails(EKeys::Gesture_LongPress, LOCTEXT("Gesture_LongPress", "Long Press"), FKeyDetails::Gesture | FKeyDetails::Axis2D, "Gesture"));

	// PS4-specific
	AddMenuCategoryDisplayInfo("Special Gamepad", LOCTEXT("SpecialGamepadSubCategory", "SpecialGamepad"), TEXT("GraphEditor.PadEvent_16x"));
	AddKey(FKeyDetails(EKeys::Gamepad_Special_Left_X, LOCTEXT("Gamepad_Special_Left_X", "Touchpad Button X Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D, "Special"));
	AddKey(FKeyDetails(EKeys::Gamepad_Special_Left_Y, LOCTEXT("Gamepad_Special_Left_Y", "Touchpad Button Y Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D, "Special"));
	AddKey(FKeyDetails(EKeys::Gamepad_Special_Left_Touched, LOCTEXT("Gamepad_Special_Left_Touched", "Touchpad Touched"), FKeyDetails::GamepadKey, "Special"));


	// Steam Controller specific
	AddMenuCategoryDisplayInfo("Steam", LOCTEXT("SteamSubCateogry", "Steam"), TEXT("GraphEditor.PadEvent_16x"));

	AddKey(FKeyDetails(EKeys::Steam_Touch_0, LOCTEXT("Steam_Touch_0", "Steam Touch 0"), FKeyDetails::GamepadKey, "Steam"));
	AddKey(FKeyDetails(EKeys::Steam_Touch_1, LOCTEXT("Steam_Touch_1", "Steam Touch 1"), FKeyDetails::GamepadKey, "Steam"));
	AddKey(FKeyDetails(EKeys::Steam_Touch_2, LOCTEXT("Steam_Touch_2", "Steam Touch 2"), FKeyDetails::GamepadKey, "Steam"));
	AddKey(FKeyDetails(EKeys::Steam_Touch_3, LOCTEXT("Steam_Touch_3", "Steam Touch 3"), FKeyDetails::GamepadKey, "Steam"));
	AddKey(FKeyDetails(EKeys::Steam_Back_Left, LOCTEXT("Steam_Back_Left", "Steam Back Left"), FKeyDetails::GamepadKey, "Steam"));
	AddKey(FKeyDetails(EKeys::Steam_Back_Right, LOCTEXT("Steam_Back_Right", "Steam Back Right"), FKeyDetails::GamepadKey, "Steam"));

	// Xbox One global speech commands
	AddMenuCategoryDisplayInfo("XBoxOne", LOCTEXT("XBoxOneSubCateogry", "XBox One"), TEXT("GraphEditor.PadEvent_16x"));

	AddKey(FKeyDetails(EKeys::Global_Menu, LOCTEXT("Global_Menu", "Global Menu"), FKeyDetails::GamepadKey, "XBoxOne"));
	AddKey(FKeyDetails(EKeys::Global_View, LOCTEXT("Global_View", "Global View"), FKeyDetails::GamepadKey, "XBoxOne"));
	AddKey(FKeyDetails(EKeys::Global_Pause, LOCTEXT("Global_Pause", "Global Pause"), FKeyDetails::GamepadKey, "XBoxOne"));
	AddKey(FKeyDetails(EKeys::Global_Play, LOCTEXT("Global_Play", "Global Play"), FKeyDetails::GamepadKey, "XBoxOne"));
	AddKey(FKeyDetails(EKeys::Global_Back, LOCTEXT("Global_Back", "Global Back"), FKeyDetails::GamepadKey, "XBoxOne"));

	// Android-specific
	AddMenuCategoryDisplayInfo("Android", LOCTEXT("AndroidSubCateogry", "Android"), TEXT("GraphEditor.KeyEvent_16x"));

	AddKey(FKeyDetails(EKeys::Android_Back, LOCTEXT("Android_Back", "Android Back"), FKeyDetails::GamepadKey, "Android"));
	AddKey(FKeyDetails(EKeys::Android_Volume_Up, LOCTEXT("Android_Volume_Up", "Android Volume Up"), FKeyDetails::GamepadKey, "Android"));
	AddKey(FKeyDetails(EKeys::Android_Volume_Down, LOCTEXT("Android_Volume_Down", "Android Volume Down"), FKeyDetails::GamepadKey, "Android"));
	AddKey(FKeyDetails(EKeys::Android_Menu, LOCTEXT("Android_Menu", "Android Menu"), FKeyDetails::GamepadKey, "Android"));

	// HTC Vive Controller
	AddMenuCategoryDisplayInfo("Vive", LOCTEXT("ViveSubCategory", "HTC Vive"), TEXT("GraphEditor.PadEvent_16x"));
	
	AddKey(FKeyDetails(EKeys::Vive_Left_Grip_Click, LOCTEXT("Vive_Left_Grip_Click", "Vive (L) Grip"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Left_Menu_Click, LOCTEXT("Vive_Left_Menu_Click", "Vive (L) Menu"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Left_Trigger_Click, LOCTEXT("Vive_Left_Trigger_Click", "Vive (L) Trigger"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Left_Trigger_Axis, LOCTEXT("Vive_Left_Trigger_Axis", "Vive (L) Trigger Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Left_Trackpad_X, LOCTEXT("Vive_Left_Trackpad_X", "Vive (L) Trackpad X-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Left_Trackpad_Y, LOCTEXT("Vive_Left_Trackpad_Y", "Vive (L) Trackpad Y-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddPairedKey(FKeyDetails(EKeys::Vive_Left_Trackpad_2D, LOCTEXT("Vive_Left_Trackpad_2D", "Vive (L) Trackpad 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D | FKeyDetails::NotBlueprintBindableKey, "Vive"), EKeys::Vive_Left_Trackpad_X, EKeys::Vive_Left_Trackpad_Y);
	AddKey(FKeyDetails(EKeys::Vive_Left_Trackpad_Click, LOCTEXT("Vive_Left_Trackpad_Click", "Vive (L) Trackpad Button"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Left_Trackpad_Touch, LOCTEXT("Vive_Left_Trackpad_Touch", "Vive (L) Trackpad Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Left_Trackpad_Up, LOCTEXT("Vive_Left_Trackpad_Up", "Vive (L) Trackpad Up"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Left_Trackpad_Down, LOCTEXT("Vive_Left_Trackpad_Down", "Vive (L) Trackpad Down"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Left_Trackpad_Left, LOCTEXT("Vive_Left_Trackpad_Left", "Vive (L) Trackpad Left"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Left_Trackpad_Right, LOCTEXT("Vive_Left_Trackpad_Right", "Vive (L) Trackpad Right"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Right_Grip_Click, LOCTEXT("Vive_Right_Grip_Click", "Vive (R) Grip"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Right_Menu_Click, LOCTEXT("Vive_Right_Menu_Click", "Vive (R) Menu"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Right_Trigger_Click, LOCTEXT("Vive_Right_Trigger_Click", "Vive (R) Trigger"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Right_Trigger_Axis, LOCTEXT("Vive_Right_Trigger_Axis", "Vive (R) Trigger Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Right_Trackpad_X, LOCTEXT("Vive_Right_Trackpad_X", "Vive (R) Trackpad X-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Right_Trackpad_Y, LOCTEXT("Vive_Right_Trackpad_Y", "Vive (R) Trackpad Y-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddPairedKey(FKeyDetails(EKeys::Vive_Right_Trackpad_2D, LOCTEXT("Vive_Right_Trackpad_2D", "Vive (R) Trackpad 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D | FKeyDetails::NotBlueprintBindableKey, "Vive"), EKeys::Vive_Right_Trackpad_X, EKeys::Vive_Right_Trackpad_Y);
	AddKey(FKeyDetails(EKeys::Vive_Right_Trackpad_Click, LOCTEXT("Vive_Right_Trackpad_Click", "Vive (R) Trackpad Button"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Right_Trackpad_Touch, LOCTEXT("Vive_Right_Trackpad_Touch", "Vive (R) Trackpad Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Right_Trackpad_Up, LOCTEXT("Vive_Right_Trackpad_Up", "Vive (R) Trackpad Up"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Right_Trackpad_Down, LOCTEXT("Vive_Right_Trackpad_Down", "Vive (R) Trackpad Down"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Right_Trackpad_Left, LOCTEXT("Vive_Right_Trackpad_Left", "Vive (R) Trackpad Left"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));
	AddKey(FKeyDetails(EKeys::Vive_Right_Trackpad_Right, LOCTEXT("Vive_Right_Trackpad_Right", "Vive (R) Trackpad Right"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "Vive"));

	// Microsoft Mixed Reality Motion Controller
	AddMenuCategoryDisplayInfo("MixedReality", LOCTEXT("MixedRealitySubCategory", "Windows Mixed Reality"), TEXT("GraphEditor.PadEvent_16x"));

	AddKey(FKeyDetails(EKeys::MixedReality_Left_Menu_Click, LOCTEXT("MixedReality_Left_Menu_Click", "Mixed Reality (L) Menu"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Grip_Click, LOCTEXT("MixedReality_Left_Grip_Click", "Mixed Reality (L) Grip"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Trigger_Click, LOCTEXT("MixedReality_Left_Trigger_Click", "Mixed Reality (L) Trigger"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Trigger_Axis, LOCTEXT("MixedReality_Left_Trigger_Axis", "Mixed Reality (L) Trigger Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Thumbstick_X, LOCTEXT("MixedReality_Left_Thumbstick_X", "Mixed Reality (L) Thumbstick X-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Thumbstick_Y, LOCTEXT("MixedReality_Left_Thumbstick_Y", "Mixed Reality (L) Thumbstick Y-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddPairedKey(FKeyDetails(EKeys::MixedReality_Left_Thumbstick_2D, LOCTEXT("MixedReality_Left_Thumbstick_2D", "Mixed Reality (L) Thumbstick 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D | FKeyDetails::NotBlueprintBindableKey, "MixedReality"), EKeys::MixedReality_Left_Thumbstick_X, EKeys::MixedReality_Left_Thumbstick_Y);
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Thumbstick_Click, LOCTEXT("MixedReality_Left_Thumbstick_Click", "Mixed Reality (L) Thumbstick Button"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Thumbstick_Up, LOCTEXT("MixedReality_Left_Thumbstick_Up", "Mixed Reality (L) Thumbstick Up"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Thumbstick_Down, LOCTEXT("MixedReality_Left_Thumbstick_Down", "Mixed Reality (L) Thumbstick Down"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Thumbstick_Left, LOCTEXT("MixedReality_Left_Thumbstick_Left", "Mixed Reality (L) Thumbstick Left"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Thumbstick_Right, LOCTEXT("MixedReality_Left_Thumbstick_Right", "Mixed Reality (L) Thumbstick Right"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Trackpad_X, LOCTEXT("MixedReality_Left_Trackpad_X", "Mixed Reality (L) Trackpad X"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Trackpad_Y, LOCTEXT("MixedReality_Left_Trackpad_Y", "Mixed Reality (L) Trackpad Y"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddPairedKey(FKeyDetails(EKeys::MixedReality_Left_Trackpad_2D, LOCTEXT("MixedReality_Left_Trackpad_2D", "Mixed Reality (L) Trackpad 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D | FKeyDetails::NotBlueprintBindableKey, "MixedReality"), EKeys::MixedReality_Left_Trackpad_X, EKeys::MixedReality_Left_Trackpad_Y);
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Trackpad_Click, LOCTEXT("MixedReality_Left_Trackpad_Click", "Mixed Reality (L) Trackpad Button"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Trackpad_Touch, LOCTEXT("MixedReality_Left_Trackpad_Touch", "Mixed Reality (L) Trackpad Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Trackpad_Up, LOCTEXT("MixedReality_Left_Trackpad_Up", "Mixed Reality (L) Trackpad Up"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Trackpad_Down, LOCTEXT("MixedReality_Left_Trackpad_Down", "Mixed Reality (L) Trackpad Down"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Trackpad_Left, LOCTEXT("MixedReality_Left_Trackpad_Left", "Mixed Reality (L) Trackpad Left"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Left_Trackpad_Right, LOCTEXT("MixedReality_Left_Trackpad_Right", "Mixed Reality (L) Trackpad Right"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Menu_Click, LOCTEXT("MixedReality_Right_Menu_Click", "Mixed Reality (R) Menu"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Grip_Click, LOCTEXT("MixedReality_Right_Grip_Click", "Mixed Reality (R) Grip"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Trigger_Click, LOCTEXT("MixedReality_Right_Trigger_Click", "Mixed Reality (R) Trigger"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Trigger_Axis, LOCTEXT("MixedReality_Right_Trigger_Axis", "Mixed Reality (R) Trigger Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Thumbstick_X, LOCTEXT("MixedReality_Right_Thumbstick_X", "Mixed Reality (R) Thumbstick X-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Thumbstick_Y, LOCTEXT("MixedReality_Right_Thumbstick_Y", "Mixed Reality (R) Thumbstick Y-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddPairedKey(FKeyDetails(EKeys::MixedReality_Right_Thumbstick_2D, LOCTEXT("MixedReality_Right_Thumbstick_2D", "Mixed Reality (R) Thumbstick 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D | FKeyDetails::NotBlueprintBindableKey, "MixedReality"), EKeys::MixedReality_Right_Thumbstick_X, EKeys::MixedReality_Right_Thumbstick_Y);
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Thumbstick_Click, LOCTEXT("MixedReality_Right_Thumbstick_Click", "Mixed Reality (R) Thumbstick Button"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Thumbstick_Up, LOCTEXT("MixedReality_Right_Thumbstick_Up", "Mixed Reality (R) Thumbstick Up"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Thumbstick_Down, LOCTEXT("MixedReality_Right_Thumbstick_Down", "Mixed Reality (R) Thumbstick Down"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Thumbstick_Left, LOCTEXT("MixedReality_Right_Thumbstick_Left", "Mixed Reality (R) Thumbstick Left"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Thumbstick_Right, LOCTEXT("MixedReality_Right_Thumbstick_Right", "Mixed Reality (R) Thumbstick Right"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Trackpad_X, LOCTEXT("MixedReality_Right_Trackpad_X", "Mixed Reality (R) Trackpad X"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Trackpad_Y, LOCTEXT("MixedReality_Right_Trackpad_Y", "Mixed Reality (R) Trackpad Y"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddPairedKey(FKeyDetails(EKeys::MixedReality_Right_Trackpad_2D, LOCTEXT("MixedReality_Right_Trackpad_2D", "Mixed Reality (R) Trackpad 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D | FKeyDetails::NotBlueprintBindableKey, "MixedReality"), EKeys::MixedReality_Right_Trackpad_X, EKeys::MixedReality_Right_Trackpad_Y);
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Trackpad_Click, LOCTEXT("MixedReality_Right_Trackpad_Click", "Mixed Reality (R) Trackpad Button"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Trackpad_Touch, LOCTEXT("MixedReality_Right_Trackpad_Touch", "Mixed Reality (R) Trackpad Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Trackpad_Up, LOCTEXT("MixedReality_Right_Trackpad_Up", "Mixed Reality (R) Trackpad Up"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Trackpad_Down, LOCTEXT("MixedReality_Right_Trackpad_Down", "Mixed Reality (R) Trackpad Down"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Trackpad_Left, LOCTEXT("MixedReality_Right_Trackpad_Left", "Mixed Reality (R) Trackpad Left"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));
	AddKey(FKeyDetails(EKeys::MixedReality_Right_Trackpad_Right, LOCTEXT("MixedReality_Right_Trackpad_Right", "Mixed Reality (R) Trackpad Right"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "MixedReality"));

	// Oculus Touch Controller
	AddMenuCategoryDisplayInfo("OculusTouch", LOCTEXT("OculusTouchSubCategory", "Oculus Touch"), TEXT("GraphEditor.PadEvent_16x"));

	AddKey(FKeyDetails(EKeys::OculusTouch_Left_X_Click, LOCTEXT("OculusTouch_Left_X_Click", "Oculus Touch (L) X Press"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Y_Click, LOCTEXT("OculusTouch_Left_Y_Click", "Oculus Touch (L) Y Press"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_X_Touch, LOCTEXT("OculusTouch_Left_X_Touch", "Oculus Touch (L) X Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Y_Touch, LOCTEXT("OculusTouch_Left_Y_Touch", "Oculus Touch (L) Y Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Menu_Click, LOCTEXT("OculusTouch_Left_Menu_Click", "Oculus Touch (L) Menu"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Grip_Click, LOCTEXT("OculusTouch_Left_Grip_Click", "Oculus Touch (L) Grip"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Grip_Axis, LOCTEXT("OculusTouch_Left_Grip_Axis", "Oculus Touch (L) Grip Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Trigger_Click, LOCTEXT("OculusTouch_Left_Trigger_Click", "Oculus Touch (L) Trigger"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Trigger_Axis, LOCTEXT("OculusTouch_Left_Trigger_Axis", "Oculus Touch (L) Trigger Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Trigger_Touch, LOCTEXT("OculusTouch_Left_Trigger_Touch", "Oculus Touch (L) Trigger Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Thumbstick_X, LOCTEXT("OculusTouch_Left_Thumbstick_X", "Oculus Touch (L) Thumbstick X-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Thumbstick_Y, LOCTEXT("OculusTouch_Left_Thumbstick_Y", "Oculus Touch (L) Thumbstick Y-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddPairedKey(FKeyDetails(EKeys::OculusTouch_Left_Thumbstick_2D, LOCTEXT("OculusTouch_Left_Thumbstick_2D", "Oculus Touch (L) Thumbstick 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"), EKeys::OculusTouch_Left_Thumbstick_X, EKeys::OculusTouch_Left_Thumbstick_Y);
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Thumbstick_Click, LOCTEXT("OculusTouch_Left_Thumbstick_Click", "Oculus Touch (L) Thumbstick Button"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Thumbstick_Touch, LOCTEXT("OculusTouch_Left_Thumbstick_Touch", "Oculus Touch (L) Thumbstick Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Thumbstick_Up, LOCTEXT("OculusTouch_Left_Thumbstick_Up", "Oculus Touch (L) Thumbstick Up"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Thumbstick_Down, LOCTEXT("OculusTouch_Left_Thumbstick_Down", "Oculus Touch (L) Thumbstick Down"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Thumbstick_Left, LOCTEXT("OculusTouch_Left_Thumbstick_Left", "Oculus Touch (L) Thumbstick Left"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Left_Thumbstick_Right, LOCTEXT("OculusTouch_Left_Thumbstick_Right", "Oculus Touch (L) Thumbstick Right"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_A_Click, LOCTEXT("OculusTouch_Right_A_Click", "Oculus Touch (R) A Press"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_B_Click, LOCTEXT("OculusTouch_Right_B_Click", "Oculus Touch (R) B Press"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_A_Touch, LOCTEXT("OculusTouch_Right_A_Touch", "Oculus Touch (R) A Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_B_Touch, LOCTEXT("OculusTouch_Right_B_Touch", "Oculus Touch (R) B Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_Grip_Click, LOCTEXT("OculusTouch_Right_Grip_Click", "Oculus Touch (R) Grip"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_Grip_Axis, LOCTEXT("OculusTouch_Right_Grip_Axis", "Oculus Touch (R) Grip Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_Trigger_Click, LOCTEXT("OculusTouch_Right_Trigger_Click", "Oculus Touch (R) Trigger"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_Trigger_Axis, LOCTEXT("OculusTouch_Right_Trigger_Axis", "Oculus Touch (R) Trigger Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_Trigger_Touch, LOCTEXT("OculusTouch_Right_Trigger_Touch", "Oculus Touch (R) Trigger Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_Thumbstick_X, LOCTEXT("OculusTouch_Right_Thumbstick_X", "Oculus Touch (R) Thumbstick X-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_Thumbstick_Y, LOCTEXT("OculusTouch_Right_Thumbstick_Y", "Oculus Touch (R) Thumbstick Y-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddPairedKey(FKeyDetails(EKeys::OculusTouch_Right_Thumbstick_2D, LOCTEXT("OculusTouch_Right_Thumbstick_2D", "Oculus Touch (R) Thumbstick 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"), EKeys::OculusTouch_Right_Thumbstick_X, EKeys::OculusTouch_Right_Thumbstick_Y);
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_Thumbstick_Click, LOCTEXT("OculusTouch_Right_Thumbstick_Click", "Oculus Touch (R) Thumbstick Button"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_Thumbstick_Touch, LOCTEXT("OculusTouch_Right_Thumbstick_Touch", "Oculus Touch (R) Thumbstick Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_Thumbstick_Up, LOCTEXT("OculusTouch_Right_Thumbstick_Up", "Oculus Touch (R) Thumbstick Up"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_Thumbstick_Down, LOCTEXT("OculusTouch_Right_Thumbstick_Down", "Oculus Touch (R) Thumbstick Down"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_Thumbstick_Left, LOCTEXT("OculusTouch_Right_Thumbstick_Left", "Oculus Touch (R) Thumbstick Left"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));
	AddKey(FKeyDetails(EKeys::OculusTouch_Right_Thumbstick_Right, LOCTEXT("OculusTouch_Right_Thumbstick_Right", "Oculus Touch (R) Thumbstick Right"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "OculusTouch"));

	// Valve Index Controller
	AddMenuCategoryDisplayInfo("ValveIndex", LOCTEXT("ValveIndexSubCategory", "Valve Index"), TEXT("GraphEditor.PadEvent_16x"));

	AddKey(FKeyDetails(EKeys::ValveIndex_Left_A_Click, LOCTEXT("ValveIndex_Left_A_Click", "Valve Index (L) A Press"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_B_Click, LOCTEXT("ValveIndex_Left_B_Click", "Valve Index (L) B Press"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_A_Touch, LOCTEXT("ValveIndex_Left_A_Touch", "Valve Index (L) A Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_B_Touch, LOCTEXT("ValveIndex_Left_B_Touch", "Valve Index (L) B Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Grip_Axis, LOCTEXT("ValveIndex_Left_Grip_Axis", "Valve Index (L) Grip Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Grip_Force, LOCTEXT("ValveIndex_Left_Grip_Force", "Valve Index (L) Grip Force"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Trigger_Click, LOCTEXT("ValveIndex_Left_Trigger_Click", "Valve Index (L) Trigger"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Trigger_Axis, LOCTEXT("ValveIndex_Left_Trigger_Axis", "Valve Index (L) Trigger Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Trigger_Touch, LOCTEXT("ValveIndex_Left_Trigger_Touch", "Valve Index (L) Trigger Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Thumbstick_X, LOCTEXT("ValveIndex_Left_Thumbstick_X", "Valve Index (L) Thumbstick X-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Thumbstick_Y, LOCTEXT("ValveIndex_Left_Thumbstick_Y", "Valve Index (L) Thumbstick Y-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddPairedKey(FKeyDetails(EKeys::ValveIndex_Left_Thumbstick_2D, LOCTEXT("ValveIndex_Left_Thumbstick_2D", "Valve Index (L) Thumbstick 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"), EKeys::ValveIndex_Left_Thumbstick_X, EKeys::ValveIndex_Left_Thumbstick_Y);
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Thumbstick_Click, LOCTEXT("ValveIndex_Left_Thumbstick_Click", "Valve Index (L) Thumbstick Button"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Thumbstick_Touch, LOCTEXT("ValveIndex_Left_Thumbstick_Touch", "Valve Index (L) Thumbstick Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Thumbstick_Up, LOCTEXT("ValveIndex_Left_Thumbstick_Up", "Valve Index (L) Thumbstick Up"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Thumbstick_Down, LOCTEXT("ValveIndex_Left_Thumbstick_Down", "Valve Index (L) Thumbstick Down"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Thumbstick_Left, LOCTEXT("ValveIndex_Left_Thumbstick_Left", "Valve Index (L) Thumbstick Left"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Thumbstick_Right, LOCTEXT("ValveIndex_Left_Thumbstick_Right", "Valve Index (L) Thumbstick Right"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Trackpad_X, LOCTEXT("ValveIndex_Left_Trackpad_X", "Valve Index (L) Trackpad X"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Trackpad_Y, LOCTEXT("ValveIndex_Left_Trackpad_Y", "Valve Index (L) Trackpad Y"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddPairedKey(FKeyDetails(EKeys::ValveIndex_Left_Trackpad_2D, LOCTEXT("ValveIndex_Left_Trackpad_2D", "Valve Index (L) Trackpad 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"), EKeys::ValveIndex_Left_Trackpad_X, EKeys::ValveIndex_Left_Trackpad_Y);
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Trackpad_Force, LOCTEXT("ValveIndex_Left_Trackpad_Force", "Valve Index (L) Trackpad Force"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Trackpad_Touch, LOCTEXT("ValveIndex_Left_Trackpad_Touch", "Valve Index (L) Trackpad Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Trackpad_Up, LOCTEXT("ValveIndex_Left_Trackpad_Up", "Valve Index (L) Trackpad Up"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Trackpad_Down, LOCTEXT("ValveIndex_Left_Trackpad_Down", "Valve Index (L) Trackpad Down"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Trackpad_Left, LOCTEXT("ValveIndex_Left_Trackpad_Left", "Valve Index (L) Trackpad Left"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Left_Trackpad_Right, LOCTEXT("ValveIndex_Left_Trackpad_Right", "Valve Index (L) Trackpad Right"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_A_Click, LOCTEXT("ValveIndex_Right_A_Click", "Valve Index (R) A Press"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_B_Click, LOCTEXT("ValveIndex_Right_B_Click", "Valve Index (R) B Press"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_A_Touch, LOCTEXT("ValveIndex_Right_A_Touch", "Valve Index (R) A Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_B_Touch, LOCTEXT("ValveIndex_Right_B_Touch", "Valve Index (R) B Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Grip_Axis, LOCTEXT("ValveIndex_Right_Grip_Axis", "Valve Index (R) Grip Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Grip_Force, LOCTEXT("ValveIndex_Right_Grip_Force", "Valve Index (R) Grip Force"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Trigger_Click, LOCTEXT("ValveIndex_Right_Trigger_Click", "Valve Index (R) Trigger"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Trigger_Axis, LOCTEXT("ValveIndex_Right_Trigger_Axis", "Valve Index (R) Trigger Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Trigger_Touch, LOCTEXT("ValveIndex_Right_Trigger_Touch", "Valve Index (R) Trigger Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Thumbstick_X, LOCTEXT("ValveIndex_Right_Thumbstick_X", "Valve Index (R) Thumbstick X-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Thumbstick_Y, LOCTEXT("ValveIndex_Right_Thumbstick_Y", "Valve Index (R) Thumbstick Y-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddPairedKey(FKeyDetails(EKeys::ValveIndex_Right_Thumbstick_2D, LOCTEXT("ValveIndex_Right_Thumbstick_2D", "Valve Index (R) Thumbstick 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"), EKeys::ValveIndex_Right_Thumbstick_X, EKeys::ValveIndex_Right_Thumbstick_Y);
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Thumbstick_Click, LOCTEXT("ValveIndex_Right_Thumbstick_Click", "Valve Index (R) Thumbstick Button"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Thumbstick_Touch, LOCTEXT("ValveIndex_Right_Thumbstick_Touch", "Valve Index (R) Thumbstick Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Thumbstick_Up, LOCTEXT("ValveIndex_Right_Thumbstick_Up", "Valve Index (R) Thumbstick Up"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Thumbstick_Down, LOCTEXT("ValveIndex_Right_Thumbstick_Down", "Valve Index (R) Thumbstick Down"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Thumbstick_Left, LOCTEXT("ValveIndex_Right_Thumbstick_Left", "Valve Index (R) Thumbstick Left"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Thumbstick_Right, LOCTEXT("ValveIndex_Right_Thumbstick_Right", "Valve Index (R) Thumbstick Right"), FKeyDetails::GamepadKey | FKeyDetails::ButtonAxis | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Trackpad_X, LOCTEXT("ValveIndex_Right_Trackpad_X", "Valve Index (R) Trackpad X"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Trackpad_Y, LOCTEXT("ValveIndex_Right_Trackpad_Y", "Valve Index (R) Trackpad Y"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddPairedKey(FKeyDetails(EKeys::ValveIndex_Right_Trackpad_2D, LOCTEXT("ValveIndex_Right_Trackpad_2D", "Valve Index (R) Trackpad 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"), EKeys::ValveIndex_Right_Trackpad_X, EKeys::ValveIndex_Right_Trackpad_Y);
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Trackpad_Force, LOCTEXT("ValveIndex_Right_Trackpad_Force", "Valve Index (R) Trackpad Force"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Trackpad_Touch, LOCTEXT("ValveIndex_Right_Trackpad_Touch", "Valve Index (R) Trackpad Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Trackpad_Up, LOCTEXT("ValveIndex_Right_Trackpad_Up", "Valve Index (R) Trackpad Up"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Trackpad_Down, LOCTEXT("ValveIndex_Right_Trackpad_Down", "Valve Index (R) Trackpad Down"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Trackpad_Left, LOCTEXT("ValveIndex_Right_Trackpad_Left", "Valve Index (R) Trackpad Left"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));
	AddKey(FKeyDetails(EKeys::ValveIndex_Right_Trackpad_Right, LOCTEXT("ValveIndex_Right_Trackpad_Right", "Valve Index (R) Trackpad Right"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "ValveIndex"));

	// Initialize the input key manager.  This will cause any additional OEM keys to get added
	FInputKeyManager::Get();
}
#if !FAST_BOOT_HACKS
UE_ENABLE_OPTIMIZATION_SHIP
#endif

void EKeys::AddKey(const FKeyDetails& KeyDetails)
{
	LLM_SCOPE_BYTAG(InputCore_EKeys);
	
	const FKey& Key = KeyDetails.GetKey();
	ensureMsgf(!InputKeys.Contains(Key), TEXT("Adding duplicate key '%s'"), *Key.ToString());
	Key.KeyDetails = MakeShareable(new FKeyDetails(KeyDetails));
	InputKeys.Add(Key, Key.KeyDetails);
}


void EKeys::AddPairedKey(const FKeyDetails& PairedKeyDetails, FKey KeyX, FKey KeyY)
{
	// Validate pairing meets all the necessary conditions.
	if (!ensureMsgf(PairedKeyDetails.IsAxis2D(), TEXT("Paired key '%s' must be a 2D axis"), *PairedKeyDetails.GetKey().ToString()) ||
		!ensureMsgf(InputKeys.Contains(KeyX), TEXT("Failed to locate key '%s' for pairing. Make sure you've bound it with AddKey before calling AddPairedKey."), *KeyX.ToString()) ||
		!ensureMsgf(InputKeys.Contains(KeyY), TEXT("Failed to locate key '%s' for pairing. Make sure you've bound it with AddKey before calling AddPairedKey."), *KeyY.ToString()) ||
		!ensureMsgf(KeyX.IsAxis1D(), TEXT("Key '%s' is not a 1D axis"), *KeyX.ToString()) ||	// TODO: Would be good to be able to pair a 2D axis to 1D axis to make a 3D axis. Possibly via another helper function?
		!ensureMsgf(KeyY.IsAxis1D(), TEXT("Key '%s' is not a 1D axis"), *KeyY.ToString()) ||
		!ensureMsgf(KeyX.GetPairedAxis() == EPairedAxis::Unpaired, TEXT("Key '%s' has been already been paired as key '%s'. Only a single pairing is permitted."), *KeyX.ToString(), *KeyX.GetPairedAxisKey().ToString()) ||
		!ensureMsgf(KeyY.GetPairedAxis() == EPairedAxis::Unpaired, TEXT("Key '%s' has been already been paired as key '%s'. Only a single pairing is permitted."), *KeyY.ToString(), *KeyY.GetPairedAxisKey().ToString())
		)
	{
		return;
	}

	// Add the basic paired key information
	AddKey(PairedKeyDetails);

	// Update key details on the paired keys to point them to the vector version

	InputKeys[KeyX]->PairedAxis = EPairedAxis::X;
	InputKeys[KeyY]->PairedAxis = EPairedAxis::Y;
	InputKeys[KeyX]->PairedAxisKey = InputKeys[KeyY]->PairedAxisKey = PairedKeyDetails.GetKey();

	PairedInputKeysLookup.Add(
		PairedKeyDetails.GetKey(),
		FPairedKeyDetails{ GetKeyDetails(KeyX), GetKeyDetails(KeyY) }
	); 
}

void EKeys::AddVirtualKey(const FKeyDetails& VirtualKeyDetails, const FKey& VirtualKeyValue)
{
	// Validate the given virtual key meets all the necessary conditions.
	if (!ensureMsgf(VirtualKeyDetails.IsVirtual(), TEXT("Paired key '%s' must be marked as a virtual key"), *VirtualKeyDetails.GetKey().ToString()) ||
		!ensureMsgf(InputKeys.Contains(VirtualKeyValue), TEXT("Failed to locate key '%s' for virtual key creation. Make sure you've bound it with AddKey before calling AddVirtualKey."), *VirtualKeyValue.ToString())
		)
	{
		return;
	}
	
	// Add the basic paired key information
	AddKey(VirtualKeyDetails);

	// Set up the virtual FKey value to be the one that was provided
	InputKeys[VirtualKeyDetails.GetKey()]->VirtualKeyValue = VirtualKeyValue;
}

void EKeys::GetAllKeys(TArray<FKey>& OutKeys)
{
	InputKeys.GetKeys(OutKeys);
}

void EKeys::RemoveKeysWithCategory(const FName InCategory)
{
	for (TMap<FKey, TSharedPtr<FKeyDetails> >::TIterator It(InputKeys); It; ++It)
	{
		FName KeyCategory = It.Key().GetMenuCategory();
		if (KeyCategory.IsEqual(InCategory))
		{
			It.RemoveCurrent();
		}
	}
}

TSharedPtr<FKeyDetails> EKeys::GetKeyDetails(const FKey Key)
{
	TSharedPtr<FKeyDetails>* KeyDetails = InputKeys.Find(Key);
	if (KeyDetails == NULL)
	{
		return TSharedPtr<FKeyDetails>();
	}
	return *KeyDetails;
}

void EKeys::AddMenuCategoryDisplayInfo(const FName CategoryName, const FText DisplayName, const FName PaletteIcon)
{
	LLM_SCOPE_BYTAG(InputCore_EKeys);
	
	if (MenuCategoryDisplayInfo.Contains(CategoryName))
	{
		UE_LOGF(LogInput, Warning, "Category %ls already has menu display info that is being replaced.", *CategoryName.ToString());
	}
	FCategoryDisplayInfo DisplayInfo;
	DisplayInfo.DisplayName = DisplayName;
	DisplayInfo.PaletteIcon = PaletteIcon;
	MenuCategoryDisplayInfo.FindOrAdd(CategoryName) = DisplayInfo;
}

FText EKeys::GetMenuCategoryDisplayName(const FName CategoryName)
{
	if (FCategoryDisplayInfo* DisplayInfo = MenuCategoryDisplayInfo.Find(CategoryName))
	{
		return DisplayInfo->DisplayName;
	}
	return MenuCategoryDisplayInfo[NAME_KeyboardCategory].DisplayName;
}

FName EKeys::GetMenuCategoryPaletteIcon(const FName CategoryName)
{
	if (FCategoryDisplayInfo* DisplayInfo = MenuCategoryDisplayInfo.Find(CategoryName))
	{
		return DisplayInfo->PaletteIcon;
	}
	return MenuCategoryDisplayInfo[NAME_KeyboardCategory].PaletteIcon;
}

EKeys::FPairedKeyDetails* EKeys::GetPairedKeyDetails(const FKey& PairedKey)
{
	return EKeys::PairedInputKeysLookup.Find(PairedKey);
}

EConsoleForGamepadLabels::Type EKeys::ConsoleForGamepadLabels = EConsoleForGamepadLabels::None;

UE_DISABLE_OPTIMIZATION_SHIP
FText EKeys::GetGamepadDisplayName(const FKey Key)
{
	switch (ConsoleForGamepadLabels)
	{
	case EConsoleForGamepadLabels::PS4:
		if (Key == EKeys::Gamepad_FaceButton_Bottom)
		{
			return LOCTEXT("PS4_Gamepad_FaceButton_Bottom", "Gamepad X");
		}
		else if (Key == EKeys::Gamepad_FaceButton_Right)
		{
			return LOCTEXT("PS4_Gamepad_FaceButton_Right", "Gamepad Circle");
		}
		else if (Key == EKeys::Gamepad_FaceButton_Left)
		{
			return LOCTEXT("PS4_Gamepad_FaceButton_Left", "Gamepad Square");
		}
		else if (Key == EKeys::Gamepad_FaceButton_Top)
		{
			return LOCTEXT("PS4_Gamepad_FaceButton_Top", "Gamepad Triangle");
		}
		else if (Key == EKeys::Gamepad_Special_Left)
		{
			return LOCTEXT("PS4_Gamepad_Special_Left", "Gamepad Touchpad Button");
		}
		else if (Key == EKeys::Gamepad_Special_Left_Touched)
		{
			return LOCTEXT("PS4_Gamepad_Special_Left_Touched", "Gamepad Touchpad Touched");
		}
		else if (Key == EKeys::Gamepad_Special_Right)
		{
			return LOCTEXT("PS4_Gamepad_Special_Right", "Gamepad Options");
		}
		else if (Key == EKeys::Gamepad_LeftShoulder)
		{
			return LOCTEXT("PS4_Gamepad_LeftShoulder", "Gamepad L1");
		}
		else if (Key == EKeys::Gamepad_RightShoulder)
		{
			return LOCTEXT("PS4_Gamepad_RightShoulder", "Gamepad R1");
		}
		else if (Key == EKeys::Gamepad_LeftTrigger)
		{
			return LOCTEXT("PS4_Gamepad_LeftTrigger", "Gamepad L2");
		}
		else if (Key == EKeys::Gamepad_RightTrigger)
		{
			return LOCTEXT("PS4_Gamepad_RightTrigger", "Gamepad R2");
		}
		else if (Key == EKeys::Gamepad_LeftTriggerAxis)
		{
			return LOCTEXT("PS4_Gamepad_LeftTriggerAxis", "Gamepad L2 Axis");
		}
		else if (Key == EKeys::Gamepad_RightTriggerAxis)
		{
			return LOCTEXT("PS4_Gamepad_RightTriggerAxis", "Gamepad R2 Axis");
		}
		else if (Key == EKeys::Gamepad_LeftThumbstick)
		{
			return LOCTEXT("PS4_Gamepad_LeftThumbstick", "Gamepad L3");
		}
		else if (Key == EKeys::Gamepad_RightThumbstick)
		{
			return LOCTEXT("PS4_Gamepad_RightThumbstick", "Gamepad R3");
		}
		break;

	case EConsoleForGamepadLabels::XBoxOne:
		if (Key == EKeys::Gamepad_FaceButton_Bottom)
		{
			return LOCTEXT("XBoxOne_Gamepad_FaceButton_Bottom", "Gamepad A");
		}
		else if (Key == EKeys::Gamepad_FaceButton_Right)
		{
			return LOCTEXT("XBoxOne_Gamepad_FaceButton_Right", "Gamepad B");
		}
		else if (Key == EKeys::Gamepad_FaceButton_Left)
		{
			return LOCTEXT("XBoxOne_Gamepad_FaceButton_Left", "Gamepad X");
		}
		else if (Key == EKeys::Gamepad_FaceButton_Top)
		{
			return LOCTEXT("XBoxOne_Gamepad_FaceButton_Top", "Gamepad Y");
		}
		else if (Key == EKeys::Gamepad_Special_Left)
		{
			return LOCTEXT("XBoxOne_Gamepad_Special_Left", "Gamepad Back");
		}
		else if (Key == EKeys::Gamepad_Special_Right)
		{
			return LOCTEXT("XBoxOne_Gamepad_Special_Right", "Gamepad Start");
		}
		else if (Key == EKeys::Gamepad_LeftShoulder)
		{
			return LOCTEXT("Gamepad_LeftShoulder", "Gamepad Left Shoulder");
		}
		else if (Key == EKeys::Gamepad_RightShoulder)
		{
			return LOCTEXT("Gamepad_RightShoulder", "Gamepad Right Shoulder");
		}
		else if (Key == EKeys::Gamepad_LeftTrigger)
		{
			return LOCTEXT("Gamepad_LeftTrigger", "Gamepad Left Trigger");
		}
		else if (Key == EKeys::Gamepad_RightTrigger)
		{
			return LOCTEXT("Gamepad_RightTrigger", "Gamepad Right Trigger");
		}
		else if (Key == EKeys::Gamepad_LeftTriggerAxis)
		{
			return LOCTEXT("Gamepad_LeftTriggerAxis", "Gamepad Left Trigger Axis");
		}
		else if (Key == EKeys::Gamepad_RightTriggerAxis)
		{
			return LOCTEXT("Gamepad_RightTriggerAxis", "Gamepad Right Trigger Axis");
		}
		else if (Key == EKeys::Gamepad_LeftThumbstick)
		{
			return LOCTEXT("Gamepad_LeftThumbstick", "Gamepad Left Thumbstick Button");
		}
		else if (Key == EKeys::Gamepad_RightThumbstick)
		{
			return LOCTEXT("Gamepad_RightThumbstick", "Gamepad Right Thumbstick Button");
		}
		break;

	default:
		if (Key == EKeys::Gamepad_FaceButton_Bottom)
		{
			return LOCTEXT("Gamepad_FaceButton_Bottom", "Gamepad Face Button Bottom");
		}
		else if (Key == EKeys::Gamepad_FaceButton_Right)
		{
			return LOCTEXT("Gamepad_FaceButton_Right", "Gamepad Face Button Right");
		}
		else if (Key == EKeys::Gamepad_FaceButton_Left)
		{
			return LOCTEXT("Gamepad_FaceButton_Left", "Gamepad Face Button Left");
		}
		else if (Key == EKeys::Gamepad_FaceButton_Top)
		{
			return LOCTEXT("Gamepad_FaceButton_Top", "Gamepad Face Button Top");
		}
		else if (Key == EKeys::Gamepad_Special_Left)
		{
			return LOCTEXT("Gamepad_Special_Left", "Gamepad Special Left");
		}
		else if (Key == EKeys::Gamepad_Special_Right)
		{
			return LOCTEXT("Gamepad_Special_Right", "Gamepad Special Right");
		}
		else if (Key == EKeys::Gamepad_LeftShoulder)
		{
			return LOCTEXT("Gamepad_LeftShoulder", "Gamepad Left Shoulder");
		}
		else if (Key == EKeys::Gamepad_RightShoulder)
		{
			return LOCTEXT("Gamepad_RightShoulder", "Gamepad Right Shoulder");
		}
		else if (Key == EKeys::Gamepad_LeftTrigger)
		{
#if WITH_EDITOR
			return LOCTEXT("Gamepad_LeftTrigger_Editor", "Gamepad Left Trigger (Digital)");
#else
			return LOCTEXT("Gamepad_LeftTrigger", "Gamepad Left Trigger");
#endif
		}
		else if (Key == EKeys::Gamepad_RightTrigger)
		{
#if WITH_EDITOR
			return LOCTEXT("Gamepad_RightTrigger_Editor", "Gamepad Right Trigger (Digital)");
#else
			return LOCTEXT("Gamepad_RightTrigger", "Gamepad Right Trigger");
#endif
		}
		else if (Key == EKeys::Gamepad_LeftTriggerAxis)
		{
			return LOCTEXT("Gamepad_LeftTriggerAxis", "Gamepad Left Trigger Axis");
		}
		else if (Key == EKeys::Gamepad_RightTriggerAxis)
		{
			return LOCTEXT("Gamepad_RightTriggerAxis", "Gamepad Right Trigger Axis");
		}
		else if (Key == EKeys::Gamepad_LeftThumbstick)
		{
			return LOCTEXT("Gamepad_LeftThumbstick", "Gamepad Left Thumbstick Button");
		}
		else if (Key == EKeys::Gamepad_RightThumbstick)
		{
			return LOCTEXT("Gamepad_RightThumbstick", "Gamepad Right Thumbstick Button");
		}
		break;
	}

	ensureMsgf(false, TEXT("Unexpected key %s using EKeys::GetGamepadDisplayName"), *Key.ToString());
	return FText::FromString(Key.ToString());
}
UE_ENABLE_OPTIMIZATION_SHIP

bool FKey::IsValid() const
{
	if (KeyName != NAME_None)
	{
		ConditionalLookupKeyDetails();
		return KeyDetails.IsValid();
	}
	return false;
}

FString FKey::ToString() const
{
	return KeyName.ToString();
}

FName FKey::GetFName() const
{
	return KeyName;
}

bool FKey::IsModifierKey() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsModifierKey() : false);
}

bool FKey::IsGamepadKey() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsGamepadKey() : false);
}

bool FKey::IsTouch() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsTouch() : false);
}

bool FKey::IsMouseButton() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsMouseButton() : false);
}

bool FKey::IsButtonAxis() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsButtonAxis() : false);
}

bool FKey::IsAxis1D() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsAxis1D() : false);
}

bool FKey::IsAxis2D() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsAxis2D() : false);
}

bool FKey::IsAxis3D() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsAxis3D() : false);
}

bool FKey::IsDigital() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsDigital() : false);
}

bool FKey::IsAnalog() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsAnalog() : false);
}

bool FKey::IsVirtual() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsVirtual() : false);
}

bool FKey::IsBindableInBlueprints() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsBindableInBlueprints() : false);
}

bool FKey::ShouldUpdateAxisWithoutSamples() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->ShouldUpdateAxisWithoutSamples() : false);
}

bool FKey::IsBindableToActions() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsBindableToActions() : false);
}

bool FKey::IsDeprecated() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsDeprecated() : false);
}

bool FKey::IsGesture() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->IsGesture() : false);
}

FText FKeyDetails::GetDisplayName(const bool bLongDisplayName) const
{
	return ((bLongDisplayName || !ShortDisplayName.IsSet()) ? LongDisplayName.Get() : ShortDisplayName.Get());
}

FText FKey::GetDisplayName(const bool bLongDisplayName) const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->GetDisplayName(bLongDisplayName) : FText::FromName(KeyName));
}

FName FKey::GetMenuCategory() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->GetMenuCategory() : EKeys::NAME_KeyboardCategory);
}

EPairedAxis FKey::GetPairedAxis() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->GetPairedAxis() : EPairedAxis::Unpaired);
}

FKey FKey::GetPairedAxisKey() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->GetPairedAxisKey() : FKey());
}

FKey FKey::GetVirtualKey() const
{
	ConditionalLookupKeyDetails();
	return (KeyDetails.IsValid() ? KeyDetails->GetVirtualKey() : FKey());
}

void FKey::ConditionalLookupKeyDetails() const
{
	if (!KeyDetails.IsValid())
	{
		KeyDetails = EKeys::GetKeyDetails(*this);
	}
}

bool FKey::IsSameResolvedKey(const FKey& Other) const
{
	// If a key is virtual, then it's FName will be different from what "KeyName" is.
	// If it is not virtual, then GetVirtualKey simply returns the normal Key implementation
	return GetVirtualKey() == Other.GetVirtualKey();
}

bool FKey::SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsEnum(TEXT("EKeys")))
	{
		Slot << KeyName;
		const FString KeyNameString(KeyName.ToString());
		const int32 FindIndex(KeyNameString.Find(TEXT("EKeys::")));
		if (FindIndex != INDEX_NONE)
		{
			KeyName = *KeyNameString.RightChop(FindIndex + 7);
			return true;
		}
	}
	else if (Tag.Type == NAME_NameProperty)
	{
		Slot << KeyName;
	}

	return false;
}

bool FKey::ExportTextItem(FString& ValueStr, FKey const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	ValueStr += KeyName.ToString();
	return true;
}

bool FKey::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	FString Temp;
	const TCHAR* NewBuffer = FPropertyHelpers::ReadToken(Buffer, Temp);
	if (!NewBuffer)
	{
		return false;
	}
	Buffer = NewBuffer;
	KeyName = *Temp;

	ResetKey();

	return true;
}

void FKey::PostSerialize(const FArchive& Ar)
{
	ResetKey();
}

void FKey::PostScriptConstruct()
{
	KeyDetails.Reset();
}

void FKey::ResetKey()
{
	KeyDetails.Reset();

	const FString KeyNameStr = KeyName.ToString();
	if (KeyNameStr.StartsWith(FKey::SyntheticCharPrefix))
	{
		// This was a dynamically added key, so we need to force it to be synthesized if it doesn't already exist
		const FString KeyNameStr2 = KeyNameStr.RightChop(FCString::Strlen(FKey::SyntheticCharPrefix));
		const uint32 CharCode = FCString::Atoi(*KeyNameStr2);
		if (CharCode > 0)
		{
			FInputKeyManager::Get().GetKeyFromCodes(INDEX_NONE, CharCode);
		}
	}
}

TSharedPtr<FInputKeyManager> FInputKeyManager::Instance;

/**
 * Returns the instance of the input key manager
 */
FInputKeyManager& FInputKeyManager::Get()
{
	LLM_SCOPE_BYTAG(InputCore_KeyManager);
	
	if( !Instance.IsValid() )
	{
		Instance = MakeShareable( new FInputKeyManager() );
	}
	return *Instance;
}

void FInputKeyManager::InitKeyMappings()
{
	LLM_SCOPE_BYTAG(InputCore_KeyManager);
	
	static const uint32 MAX_KEY_MAPPINGS(256);
	uint32 KeyCodes[MAX_KEY_MAPPINGS], CharCodes[MAX_KEY_MAPPINGS];
	FString KeyNames[MAX_KEY_MAPPINGS], CharKeyNames[MAX_KEY_MAPPINGS];

	uint32 const CharKeyMapSize(FPlatformInput::GetCharKeyMap(CharCodes, CharKeyNames, MAX_KEY_MAPPINGS));
	uint32 const KeyMapSize(FPlatformInput::GetKeyMap(KeyCodes, KeyNames, MAX_KEY_MAPPINGS));

	// When the input language changes, a key that was virtual may no longer be virtual.
	// We must repopulate the maps to ensure GetKeyFromCodes returns the correct value per language.
	KeyMapVirtualToEnum.Reset();
	KeyMapCharToEnum.Reset();
	for (uint32 Idx=0; Idx<KeyMapSize; ++Idx)
	{
		FKey Key(*KeyNames[Idx]);

		if (!Key.IsValid())
		{
			EKeys::AddKey(FKeyDetails(Key, Key.GetDisplayName()));
		}

		KeyMapVirtualToEnum.Add(KeyCodes[Idx], Key);
	}
	for (uint32 Idx=0; Idx<CharKeyMapSize; ++Idx)
	{
		// repeated linear search here isn't ideal, but it's just once at startup
		const FKey Key(*CharKeyNames[Idx]);

		if (ensureMsgf(Key.IsValid(), TEXT("Failed to get key for name %s"), *CharKeyNames[Idx]))
		{
			KeyMapCharToEnum.Add(CharCodes[Idx], Key);
		}
	}
}

/**
 * Retrieves the key mapped to the specified key or character code.
 *
 * @param	KeyCode	the key code to get the name for
 */
FKey FInputKeyManager::GetKeyFromCodes( const uint32 KeyCode, const uint32 CharCode ) const
{
	const FKey* KeyPtr(KeyMapVirtualToEnum.Find(KeyCode));
	if (KeyPtr == nullptr)
	{
		KeyPtr = KeyMapCharToEnum.Find(CharCode);
	}
	// If we didn't find a FKey and the CharCode is not a control character (using 32/space as the start of that range),
	// then we want to synthesize a new FKey for this unknown character so that key binding on non-qwerty keyboards works better
	if (KeyPtr == nullptr && CharCode > 32)
	{
		FKey NewKey(*FString::Printf(TEXT("%s%d"), FKey::SyntheticCharPrefix, CharCode));
		EKeys::AddKey(FKeyDetails(NewKey, FText::AsCultureInvariant(FString::Chr((TCHAR)CharCode)), FKeyDetails::NotBlueprintBindableKey | FKeyDetails::NotActionBindableKey));
		const_cast<FInputKeyManager*>(this)->KeyMapCharToEnum.Add(CharCode, NewKey);
		return NewKey;
	}
	return KeyPtr ? *KeyPtr : EKeys::Invalid;
}

void FInputKeyManager::GetCodesFromKey(const FKey Key, const uint32*& KeyCode, const uint32*& CharCode) const
{
	CharCode = KeyMapCharToEnum.FindKey(Key);
	KeyCode = KeyMapVirtualToEnum.FindKey(Key);
}

#undef LOCTEXT_NAMESPACE

