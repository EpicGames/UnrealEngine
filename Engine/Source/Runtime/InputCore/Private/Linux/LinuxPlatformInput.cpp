// Copyright Epic Games, Inc. All Rights Reserved.

#include "Linux/LinuxPlatformInput.h"

THIRD_PARTY_INCLUDES_START
	#include <SDL3/SDL.h>
THIRD_PARTY_INCLUDES_END

uint32 FLinuxPlatformInput::GetKeyMap( uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings )
{
#define ADDKEYMAP(KeyCode, KeyName)		if (NumMappings<MaxMappings) { KeyCodes[NumMappings]=KeyCode; KeyNames[NumMappings]=KeyName; ++NumMappings; };

	uint32 NumMappings = 0;

	if (KeyCodes && KeyNames && (MaxMappings > 0))
	{
		ADDKEYMAP(SDLK_BACKSPACE, TEXT("BackSpace"));
		ADDKEYMAP(SDLK_TAB, TEXT("Tab"));
		ADDKEYMAP(SDLK_RETURN, TEXT("Enter"));
		ADDKEYMAP(SDLK_RETURN2, TEXT("Enter"));
		ADDKEYMAP(SDLK_KP_ENTER, TEXT("Enter"));
		ADDKEYMAP(SDLK_PAUSE, TEXT("Pause"));

		ADDKEYMAP(SDLK_ESCAPE, TEXT("Escape"));
		ADDKEYMAP(SDLK_SPACE, TEXT("SpaceBar"));
		ADDKEYMAP(SDLK_PAGEUP, TEXT("PageUp"));
		ADDKEYMAP(SDLK_PAGEDOWN, TEXT("PageDown"));
		ADDKEYMAP(SDLK_END, TEXT("End"));
		ADDKEYMAP(SDLK_HOME, TEXT("Home"));

		ADDKEYMAP(SDLK_LEFT, TEXT("Left"));
		ADDKEYMAP(SDLK_UP, TEXT("Up"));
		ADDKEYMAP(SDLK_RIGHT, TEXT("Right"));
		ADDKEYMAP(SDLK_DOWN, TEXT("Down"));

		ADDKEYMAP(SDLK_INSERT, TEXT("Insert"));
		ADDKEYMAP(SDLK_DELETE, TEXT("Delete"));

		ADDKEYMAP(SDLK_F1, TEXT("F1"));
		ADDKEYMAP(SDLK_F2, TEXT("F2"));
		ADDKEYMAP(SDLK_F3, TEXT("F3"));
		ADDKEYMAP(SDLK_F4, TEXT("F4"));
		ADDKEYMAP(SDLK_F5, TEXT("F5"));
		ADDKEYMAP(SDLK_F6, TEXT("F6"));
		ADDKEYMAP(SDLK_F7, TEXT("F7"));
		ADDKEYMAP(SDLK_F8, TEXT("F8"));
		ADDKEYMAP(SDLK_F9, TEXT("F9"));
		ADDKEYMAP(SDLK_F10, TEXT("F10"));
		ADDKEYMAP(SDLK_F11, TEXT("F11"));
		ADDKEYMAP(SDLK_F12, TEXT("F12"));

		ADDKEYMAP(SDLK_LCTRL, TEXT("LeftControl"));
		ADDKEYMAP(SDLK_LSHIFT, TEXT("LeftShift"));
		ADDKEYMAP(SDLK_LALT, TEXT("LeftAlt"));
		ADDKEYMAP(SDLK_LGUI, TEXT("LeftCommand"));
		ADDKEYMAP(SDLK_RCTRL, TEXT("RightControl"));
		ADDKEYMAP(SDLK_RSHIFT, TEXT("RightShift"));
		ADDKEYMAP(SDLK_RALT, TEXT("RightAlt"));
		ADDKEYMAP(SDLK_RGUI, TEXT("RightCommand"));

		ADDKEYMAP(SDLK_KP_0, TEXT("NumPadZero"));
		ADDKEYMAP(SDLK_KP_1, TEXT("NumPadOne"));
		ADDKEYMAP(SDLK_KP_2, TEXT("NumPadTwo"));
		ADDKEYMAP(SDLK_KP_3, TEXT("NumPadThree"));
		ADDKEYMAP(SDLK_KP_4, TEXT("NumPadFour"));
		ADDKEYMAP(SDLK_KP_5, TEXT("NumPadFive"));
		ADDKEYMAP(SDLK_KP_6, TEXT("NumPadSix"));
		ADDKEYMAP(SDLK_KP_7, TEXT("NumPadSeven"));
		ADDKEYMAP(SDLK_KP_8, TEXT("NumPadEight"));
		ADDKEYMAP(SDLK_KP_9, TEXT("NumPadNine"));
		ADDKEYMAP(SDLK_KP_MULTIPLY, TEXT("Multiply"));
		ADDKEYMAP(SDLK_KP_PLUS, TEXT("Add"));
		ADDKEYMAP(SDLK_KP_MINUS, TEXT("Subtract"));
		ADDKEYMAP(SDLK_KP_DECIMAL, TEXT("Decimal"));
		ADDKEYMAP(SDLK_KP_DIVIDE, TEXT("Divide"));

		ADDKEYMAP(SDLK_CAPSLOCK, TEXT("CapsLock"));
		ADDKEYMAP(SDLK_NUMLOCKCLEAR, TEXT("NumLock"));
		ADDKEYMAP(SDLK_SCROLLLOCK, TEXT("ScrollLock"));
	}
#undef ADDKEYMAP

	check(NumMappings < MaxMappings);
	return NumMappings;
}

uint32 FLinuxPlatformInput::GetCharKeyMap(uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings)
{
	return FGenericPlatformInput::GetStandardPrintableKeyMap(KeyCodes, KeyNames, MaxMappings, false, true);
}
