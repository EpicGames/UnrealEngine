// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacPlatformInput.h"
#include "Misc/ConfigCacheIni.h"	// For GConfig
#include "Mac/MacSystemIncludes.h"

uint32 FMacPlatformInput::GetKeyMap( uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings )
{
#define ADDKEYMAP(KeyCode, KeyName)		if (NumMappings<MaxMappings) { KeyCodes[NumMappings]=KeyCode; KeyNames[NumMappings]=KeyName; ++NumMappings; };

	uint32 NumMappings = 0;

	if ( KeyCodes && KeyNames && (MaxMappings > 0) )
	{
		ADDKEYMAP( kVK_Delete, TEXT("BackSpace") );
		ADDKEYMAP( kVK_Tab, TEXT("Tab") );
		ADDKEYMAP( kVK_Return, TEXT("Enter") );
		ADDKEYMAP( kVK_ANSI_KeypadEnter, TEXT("Enter") );

		ADDKEYMAP( kVK_CapsLock, TEXT("CapsLock") );
		ADDKEYMAP( kVK_Escape, TEXT("Escape") );
		ADDKEYMAP( kVK_Space, TEXT("SpaceBar") );
		ADDKEYMAP( kVK_PageUp, TEXT("PageUp") );
		ADDKEYMAP( kVK_PageDown, TEXT("PageDown") );
		ADDKEYMAP( kVK_End, TEXT("End") );
		ADDKEYMAP( kVK_Home, TEXT("Home") );

		ADDKEYMAP( kVK_LeftArrow, TEXT("Left") );
		ADDKEYMAP( kVK_UpArrow, TEXT("Up") );
		ADDKEYMAP( kVK_RightArrow, TEXT("Right") );
		ADDKEYMAP( kVK_DownArrow, TEXT("Down") );

		ADDKEYMAP( kVK_ForwardDelete, TEXT("Delete") );

		ADDKEYMAP( kVK_ANSI_Keypad0, TEXT("NumPadZero") );
		ADDKEYMAP( kVK_ANSI_Keypad1, TEXT("NumPadOne") );
		ADDKEYMAP( kVK_ANSI_Keypad2, TEXT("NumPadTwo") );
		ADDKEYMAP( kVK_ANSI_Keypad3, TEXT("NumPadThree") );
		ADDKEYMAP( kVK_ANSI_Keypad4, TEXT("NumPadFour") );
		ADDKEYMAP( kVK_ANSI_Keypad5, TEXT("NumPadFive") );
		ADDKEYMAP( kVK_ANSI_Keypad6, TEXT("NumPadSix") );
		ADDKEYMAP( kVK_ANSI_Keypad7, TEXT("NumPadSeven") );
		ADDKEYMAP( kVK_ANSI_Keypad8, TEXT("NumPadEight") );
		ADDKEYMAP( kVK_ANSI_Keypad9, TEXT("NumPadNine") );

		ADDKEYMAP( kVK_ANSI_KeypadMultiply, TEXT("Multiply") );
		ADDKEYMAP( kVK_ANSI_KeypadPlus, TEXT("Add") );
		ADDKEYMAP( kVK_ANSI_KeypadMinus, TEXT("Subtract") );
		ADDKEYMAP( kVK_ANSI_KeypadDecimal, TEXT("Decimal") );
		ADDKEYMAP( kVK_ANSI_KeypadDivide, TEXT("Divide") );

		ADDKEYMAP( kVK_F1, TEXT("F1") );
		ADDKEYMAP( kVK_F2, TEXT("F2") );
		ADDKEYMAP( kVK_F3, TEXT("F3") );
		ADDKEYMAP( kVK_F4, TEXT("F4") );
		ADDKEYMAP( kVK_F5, TEXT("F5") );
		ADDKEYMAP( kVK_F6, TEXT("F6") );
		ADDKEYMAP( kVK_F7, TEXT("F7") );
		ADDKEYMAP( kVK_F8, TEXT("F8") );
		ADDKEYMAP( kVK_F9, TEXT("F9") );
		ADDKEYMAP( kVK_F10, TEXT("F10") );
		ADDKEYMAP( kVK_F11, TEXT("F11") );
		ADDKEYMAP( kVK_F12, TEXT("F12") );

		ADDKEYMAP( MMK_RightControl, TEXT("RightControl") );
		ADDKEYMAP( MMK_LeftControl, TEXT("LeftControl") );
		ADDKEYMAP( MMK_LeftShift, TEXT("LeftShift") );
		ADDKEYMAP( MMK_CapsLock, TEXT("CapsLock") );
		ADDKEYMAP( MMK_LeftAlt, TEXT("LeftAlt") );
		ADDKEYMAP( MMK_LeftCommand, TEXT("LeftCommand") );
		ADDKEYMAP( MMK_RightShift, TEXT("RightShift") );
		ADDKEYMAP( MMK_RightAlt, TEXT("RightAlt") );
		ADDKEYMAP( MMK_RightCommand, TEXT("RightCommand") );
	}
#undef ADDKEYMAP

	check(NumMappings < MaxMappings);
	return NumMappings;
}

uint32 FMacPlatformInput::GetCharKeyMap(uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings)
{
	return FGenericPlatformInput::GetStandardPrintableKeyMap(KeyCodes, KeyNames, MaxMappings, false, true);
}

FKey FMacPlatformInput::GetPlatformDeleteKey()
{
	// The legacy behavior is that Platform Delete is "Backspace". In 5.4 we added this option to have it
	// be the normal "Delete" key. 
	bool bPlatformDeleteIsBackspace = true;
	if (GConfig)
	{
		GConfig->GetBool(TEXT("MacInput"), TEXT("bPlatformDeleteIsBackspace"), bPlatformDeleteIsBackspace, GInputIni);
	}

	return bPlatformDeleteIsBackspace ? EKeys::BackSpace : EKeys::Delete;
}

uint32 FMacPlatformInput::GetMacNativeModifierKeyCode(uint32 ModifierKeyCode)
{
	switch(ModifierKeyCode) 
	{
		case MMK_RightCommand:
		case MMK_LeftCommand:
			return kVK_Command;
		case MMK_LeftShift:
			return kVK_Shift;
		case MMK_CapsLock:
			return kVK_CapsLock;
		case MMK_LeftAlt:
			return kVK_Option;
		case MMK_LeftControl:
			return kVK_Control;
		case MMK_RightShift:
			return kVK_RightShift;
		case MMK_RightAlt:
			return kVK_RightOption;
		case MMK_RightControl:
			return kVK_RightControl;
		default:
			return ModifierKeyCode;
	}
}
