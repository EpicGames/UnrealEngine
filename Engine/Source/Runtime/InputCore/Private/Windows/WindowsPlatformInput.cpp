// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsPlatformInput.h"
#include "Windows/WindowsHWrapper.h"

uint32 FWindowsPlatformInput::GetKeyMap( uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings )
{
#define ADDKEYMAP(KeyCode, KeyName)		if (NumMappings<MaxMappings) { KeyCodes[NumMappings]=KeyCode; KeyNames[NumMappings]=KeyName; ++NumMappings; };

	uint32 NumMappings = 0;

	if ( KeyCodes && KeyNames && (MaxMappings > 0) )
	{
		ADDKEYMAP( VK_LBUTTON, TEXT("LeftMouseButton") );
		ADDKEYMAP( VK_RBUTTON, TEXT("RightMouseButton") );
		ADDKEYMAP( VK_MBUTTON, TEXT("MiddleMouseButton") );

		ADDKEYMAP( VK_XBUTTON1, TEXT("ThumbMouseButton") );
		ADDKEYMAP( VK_XBUTTON2, TEXT("ThumbMouseButton2") );

		ADDKEYMAP( VK_BACK, TEXT("BackSpace") );
		ADDKEYMAP( VK_TAB, TEXT("Tab") );
		ADDKEYMAP( VK_RETURN, TEXT("Enter") );
		ADDKEYMAP( VK_PAUSE, TEXT("Pause") );

		ADDKEYMAP( VK_CAPITAL, TEXT("CapsLock") );
		ADDKEYMAP( VK_ESCAPE, TEXT("Escape") );
		ADDKEYMAP( VK_SPACE, TEXT("SpaceBar") );
		ADDKEYMAP( VK_PRIOR, TEXT("PageUp") );
		ADDKEYMAP( VK_NEXT, TEXT("PageDown") );
		ADDKEYMAP( VK_END, TEXT("End") );
		ADDKEYMAP( VK_HOME, TEXT("Home") );

		ADDKEYMAP( VK_LEFT, TEXT("Left") );
		ADDKEYMAP( VK_UP, TEXT("Up") );
		ADDKEYMAP( VK_RIGHT, TEXT("Right") );
		ADDKEYMAP( VK_DOWN, TEXT("Down") );

		ADDKEYMAP( VK_INSERT, TEXT("Insert") );
		ADDKEYMAP( VK_DELETE, TEXT("Delete") );

		ADDKEYMAP( VK_NUMPAD0, TEXT("NumPadZero") );
		ADDKEYMAP( VK_NUMPAD1, TEXT("NumPadOne") );
		ADDKEYMAP( VK_NUMPAD2, TEXT("NumPadTwo") );
		ADDKEYMAP( VK_NUMPAD3, TEXT("NumPadThree") );
		ADDKEYMAP( VK_NUMPAD4, TEXT("NumPadFour") );
		ADDKEYMAP( VK_NUMPAD5, TEXT("NumPadFive") );
		ADDKEYMAP( VK_NUMPAD6, TEXT("NumPadSix") );
		ADDKEYMAP( VK_NUMPAD7, TEXT("NumPadSeven") );
		ADDKEYMAP( VK_NUMPAD8, TEXT("NumPadEight") );
		ADDKEYMAP( VK_NUMPAD9, TEXT("NumPadNine") );

		ADDKEYMAP( VK_MULTIPLY, TEXT("Multiply") );
		ADDKEYMAP( VK_ADD, TEXT("Add") );
		ADDKEYMAP( VK_SUBTRACT, TEXT("Subtract") );
		ADDKEYMAP( VK_DECIMAL, TEXT("Decimal") );
		ADDKEYMAP( VK_DIVIDE, TEXT("Divide") );

		ADDKEYMAP( VK_F1, TEXT("F1") );
		ADDKEYMAP( VK_F2, TEXT("F2") );
		ADDKEYMAP( VK_F3, TEXT("F3") );
		ADDKEYMAP( VK_F4, TEXT("F4") );
		ADDKEYMAP( VK_F5, TEXT("F5") );
		ADDKEYMAP( VK_F6, TEXT("F6") );
		ADDKEYMAP( VK_F7, TEXT("F7") );
		ADDKEYMAP( VK_F8, TEXT("F8") );
		ADDKEYMAP( VK_F9, TEXT("F9") );
		ADDKEYMAP( VK_F10, TEXT("F10") );
		ADDKEYMAP( VK_F11, TEXT("F11") );
		ADDKEYMAP( VK_F12, TEXT("F12") );

		ADDKEYMAP( VK_NUMLOCK, TEXT("NumLock") );

		ADDKEYMAP( VK_SCROLL, TEXT("ScrollLock") );

		ADDKEYMAP( VK_LSHIFT, TEXT("LeftShift") );
		ADDKEYMAP( VK_RSHIFT, TEXT("RightShift") );
		ADDKEYMAP( VK_LCONTROL, TEXT("LeftControl") );
		ADDKEYMAP( VK_RCONTROL, TEXT("RightControl") );
		ADDKEYMAP( VK_LMENU, TEXT("LeftAlt") );
		ADDKEYMAP( VK_RMENU, TEXT("RightAlt") );
		ADDKEYMAP( VK_LWIN, TEXT("LeftCommand") );
		ADDKEYMAP( VK_RWIN, TEXT("RightCommand") );

		TMap<uint32, uint32> ScanToVKMap;
#define MAP_OEM_VK_TO_SCAN(KeyCode) { const uint32 CharCode = MapVirtualKey(KeyCode,2); if (CharCode != 0) { ScanToVKMap.Add(CharCode,KeyCode); } }
		MAP_OEM_VK_TO_SCAN(VK_OEM_1);
		MAP_OEM_VK_TO_SCAN(VK_OEM_2);
		MAP_OEM_VK_TO_SCAN(VK_OEM_3);
		MAP_OEM_VK_TO_SCAN(VK_OEM_4);
		MAP_OEM_VK_TO_SCAN(VK_OEM_5);
		MAP_OEM_VK_TO_SCAN(VK_OEM_6);
		MAP_OEM_VK_TO_SCAN(VK_OEM_7);
		MAP_OEM_VK_TO_SCAN(VK_OEM_8);
		MAP_OEM_VK_TO_SCAN(VK_OEM_PLUS);
		MAP_OEM_VK_TO_SCAN(VK_OEM_COMMA);
		MAP_OEM_VK_TO_SCAN(VK_OEM_MINUS);
		MAP_OEM_VK_TO_SCAN(VK_OEM_PERIOD);
		MAP_OEM_VK_TO_SCAN(VK_OEM_102);
#undef  MAP_OEM_VK_TO_SCAN

		static const uint32 MAX_KEY_MAPPINGS(256);
		uint32 CharCodes[MAX_KEY_MAPPINGS];
		FString CharKeyNames[MAX_KEY_MAPPINGS];
		const int32 CharMappings = GetCharKeyMap(CharCodes, CharKeyNames, MAX_KEY_MAPPINGS);

		for (int32 MappingIndex = 0; MappingIndex < CharMappings; ++MappingIndex)
		{
			ScanToVKMap.Remove(CharCodes[MappingIndex]);
		}

		for (auto It(ScanToVKMap.CreateConstIterator()); It; ++It)
		{
			ADDKEYMAP(It.Value(), FString::Chr((TCHAR)It.Key()));
		}
	}

	check(NumMappings < MaxMappings);
	return NumMappings;

#undef ADDKEYMAP
}

uint32 FWindowsPlatformInput::GetCharKeyMap(uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings)
{
	return FGenericPlatformInput::GetStandardPrintableKeyMap(KeyCodes, KeyNames, MaxMappings, true, false);
}

