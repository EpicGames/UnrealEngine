// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformInput.h"

struct FAndroidPlatformInput : FGenericPlatformInput
{
	static INPUTCORE_API uint32 GetCharKeyMap(uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings);
	static INPUTCORE_API uint32 GetKeyMap( uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings );
};

typedef FAndroidPlatformInput FPlatformInput;
