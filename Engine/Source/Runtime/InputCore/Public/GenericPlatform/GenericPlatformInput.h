// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "InputCoreTypes.h"

class FString;

struct FGenericPlatformInput
{
public:
	inline static uint32 GetKeyMap( uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings )
	{
		return 0;
	}

	inline static uint32 GetCharKeyMap(uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings)
	{
		return 0;
	}

	/** Remap a given key to something else if necessary. */
	inline static FKey RemapKey(FKey Key) { return Key; }

	static FKey GetGamepadAcceptKey()
	{
		return EKeys::Gamepad_FaceButton_Bottom;
	}

	static FKey GetGamepadBackKey()
	{
		return EKeys::Gamepad_FaceButton_Right;
	}

	static FKey GetPlatformDeleteKey()
	{
		return EKeys::Delete;
	}

protected:
	/**
	* Retrieves some standard key code mappings (usually called by a subclass's GetCharKeyMap)
	*
	* @param OutKeyMap Key map to add to.
	* @param bMapUppercaseKeys If true, will map A, B, C, etc to EKeys::A, EKeys::B, EKeys::C
	* @param bMapLowercaseKeys If true, will map a, b, c, etc to EKeys::A, EKeys::B, EKeys::C
	*/
	static INPUTCORE_API uint32 GetStandardPrintableKeyMap(uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings, bool bMapUppercaseKeys, bool bMapLowercaseKeys);
};
