// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "Modules/ModuleInterface.h"

/**
 * The public interface of the InputDeviceModule
 */
class IInputDebuggingInterface : public IModuleInterface, public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("InputDebuggingInterface");
		return FeatureName;
	}

	static inline bool IsAvailable()
	{
		return IModularFeatures::Get().IsModularFeatureAvailable(GetModularFeatureName());
	}

	static inline IInputDebuggingInterface& Get()
	{
		return IModularFeatures::Get().GetModularFeature<IInputDebuggingInterface>(GetModularFeatureName());
	}
};
