// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class FInputCoreModule : public IModuleInterface
{
public:

	// IModuleInterface

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 * Overloaded to allow the default subsystem a chance to load
	 */
	virtual void StartupModule() override;
};
