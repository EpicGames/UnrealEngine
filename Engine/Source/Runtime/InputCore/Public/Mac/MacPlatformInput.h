// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformInput.h"

struct INPUTCORE_API FMacPlatformInput : FGenericPlatformInput
{
	static uint32 GetKeyMap( uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings );
	static uint32 GetCharKeyMap(uint32* KeyCodes, FString* KeyNames, uint32 MaxMappings);
	static FKey GetPlatformDeleteKey();
	static uint32 GetMacNativeModifierKeyCode(uint32 ModifierKeyCode);
};

typedef FMacPlatformInput FPlatformInput;
